{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.3",
	"FriendlyName": "ML Deformer Nearest Neighbor Model (DEPRECATED)",
	"Description": "Nearest Neighbor Model for the ML Deformer Framework. This model has been deprecated. Please use the Detail Pose Model instead.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/5.0/en-US/using-the-machine-learning-deformer-in-unreal-engine/",
	"MarketplaceURL": "",
	"SupportURL": "https://forums.unrealengine.com/",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "NearestNeighborModel",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NearestNeighborModelEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "ComputeFramework",
			"Enabled": true
		},
		{
			"Name": "DeformerGraph",
			"Enabled": true
		},
		{
			"Name": "PythonMLPackages",
			"Enabled": true
		},
		{
			"Name": "MLDeformerFramework",
			"Enabled": true
		},
		{
			"Name": "NNERuntimeBasicCpu",
			"Enabled": true
		}
	],
	"PythonRequirements":
	[
		{
			"Platform": "All",
			"Requirements":
			[
				"joblib==1.2.0 --hash=sha256:091138ed78f800342968c523bdde947e7a305b8594b910a0fea2ab83c3c6d385",
				"scikit-learn==1.2.1 --hash=sha256:d00e46a2a7fce6e118ed0f4c6263785bf6c297a94ffd0cd7b32455043c508cc8 --hash=sha256:c9285275a435d1f8f47bbe3500346ab9ead2499e0e090518404d318ea90d1c1c --hash=sha256:da0e2d50a8435ea8dc5cd21f1fc1a45d329bae03dcca92087ebed859d22d184e --hash=sha256:5a8111f3c7a314017ebf90d6feab861c11d1ca14f3dbafb39abcc31aa4c54ba6 --hash=sha256:c722f3446ad8c4f1a93b2399fe1a188635b94709a3f25e6f4d61efbe75fe8eaa --hash=sha256:e0ee4d4d32c94e082344308528f7b3c9294b60ab19c84eb37a2d9c88bdffd9d1 --hash=sha256:d8bcd303dd982494842a3f482f844d539484c6043b4eed896b43ea8e5f609a21 --hash=sha256:5523e21ab2b4d52b2bd41bedd335dbe8f3c1b5f6dd7c9c001b2e17ec9818af8d --hash=sha256:70fa30d146b7e9d0c256e73e271b3e17f23123b7c4adcbde1a385031adf59090 --hash=sha256:bed9f75763bd392c094bf474c7ab75a01d68b15146ea7a20c0f9ff6fb3063dad --hash=sha256:479aedd0abedbda6b8b4529145fe4cd8622f69f726a72cef8f75548a93eeb1e1 --hash=sha256:a9abf17d177df54e529154f26acfd42930e19117d045e8a9a8e893ca82dd94ec",
				"threadpoolctl==3.1.0 --hash=sha256:8b99adda265feb6773280df41eece7b2e6561b772d21ffd52e372f999024907b"
			]
		}
	]
}