// Copyright Epic Games, Inc. All Rights Reserved.

uint {DataInterfaceName}_NumVertices;
uint {DataInterfaceName}_NumBoneInfluences;
uint {DataInterfaceName}_InputWeightStride;
uint {DataInterfaceName}_InputWeightIndexSize;
FCompressedBoneTransformBuffer {DataInterfaceName}_BoneMatrices;
Buffer<float4> {DataInterfaceName}_BoneQuats;
Buffer<uint> {DataInterfaceName}_InputWeightStream;
Buffer<uint> {DataInterfaceName}_InputWeightLookupStream;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

#if !GPUSKIN_UNLIMITED_BONE_INFLUENCE

int4 GetBlendIndices_{DataInterfaceName}(uint StreamOffset, uint ExtraInfluenceIndex)
{
	return GetBlendIndices({DataInterfaceName}_InputWeightStream, StreamOffset, ExtraInfluenceIndex);
}

float4 GetBlendWeights_{DataInterfaceName}(uint StreamOffset, uint ExtraInfluenceIndex)
{
	return GetBlendWeights({DataInterfaceName}_InputWeightStream, StreamOffset, ExtraInfluenceIndex, {DataInterfaceName}_NumBoneInfluences);
}

#endif // GPUSKIN_UNLIMITED_BONE_INFLUENCE

float3x4 GetBoneMatrix_{DataInterfaceName}(uint BoneIndex)
{
	return LoadCompressedBoneTransformTransposed({DataInterfaceName}_BoneMatrices, 0, BoneIndex);
}

uint ReadNumBones_{DataInterfaceName}(uint VertexIndex)
{
#if !ENABLE_DEFORMER_BONES
	return 1;
#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE
	uint BlendOffsetCount = {DataInterfaceName}_InputWeightLookupStream[VertexIndex];
	return BlendOffsetCount & 0xff;
#else // !GPUSKIN_UNLIMITED_BONE_INFLUENCE
	return {DataInterfaceName}_NumBoneInfluences;
#endif
}

float3x4 ReadBoneMatrix_{DataInterfaceName}(uint VertexIndex, uint BoneIndex)
{
#if !ENABLE_DEFORMER_BONES

	return float3x4(1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0);

#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE
	
	uint BlendOffsetCount = {DataInterfaceName}_InputWeightLookupStream[VertexIndex];
	int StreamOffset = BlendOffsetCount >> 8;

	int BoneIndexOffset = StreamOffset + ({DataInterfaceName}_InputWeightIndexSize * BoneIndex);
	int BoneGlobalIndex = {DataInterfaceName}_InputWeightStream[BoneIndexOffset];
	if ({DataInterfaceName}_InputWeightIndexSize > 1)
	{
		BoneGlobalIndex = {DataInterfaceName}_InputWeightStream[BoneIndexOffset + 1] << 8 | BoneGlobalIndex;
	}
	
	return GetBoneMatrix_{DataInterfaceName}(BoneGlobalIndex);

#else // !GPUSKIN_UNLIMITED_BONE_INFLUENCE
	
	uint StreamOffset = (VertexIndex * ({DataInterfaceName}_InputWeightStride / 4));
	int BlendIndex = GetBlendIndices_{DataInterfaceName}(StreamOffset, BoneIndex / 4)[BoneIndex & 0x3];
	return GetBoneMatrix_{DataInterfaceName}(BlendIndex);

#endif
}

float ReadBoneWeight_{DataInterfaceName}(uint VertexIndex, uint BoneIndex)
{
#if !ENABLE_DEFORMER_BONES

	return 1;

#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE
	
	uint BlendOffsetCount = {DataInterfaceName}_InputWeightLookupStream[VertexIndex];
	int NumBoneInfluencesLocal = BlendOffsetCount & 0xff;
	int StreamOffset = BlendOffsetCount >> 8;
	int BoneIndexSize = {DataInterfaceName}_InputWeightIndexSize & 0xff;
	int BoneWeightSize = {DataInterfaceName}_InputWeightIndexSize >> 8;
	int WeightsOffset = StreamOffset + (BoneIndexSize * NumBoneInfluencesLocal);
	int BoneWeightOffset = WeightsOffset + BoneIndex * BoneWeightSize;
	
	if (BoneWeightSize > 1)
	{
		return float({DataInterfaceName}_InputWeightStream[BoneWeightOffset + 1] << 8 | {DataInterfaceName}_InputWeightStream[BoneWeightOffset]) / 65535.0;
	}
	else
	{
		return float({DataInterfaceName}_InputWeightStream[BoneWeightOffset]) / 255.0f;
	}

#else // !GPUSKIN_UNLIMITED_BONE_INFLUENCE
	
	uint StreamOffset = (VertexIndex * ({DataInterfaceName}_InputWeightStride / 4));
	float BlendWeight = GetBlendWeights_{DataInterfaceName}(StreamOffset, BoneIndex / 4)[BoneIndex & 0x3];
	return BlendWeight;

#endif
}

float3x4 ReadWeightedBoneMatrix_{DataInterfaceName}(uint VertexIndex)
{
#if !ENABLE_DEFORMER_BONES

	return float3x4(1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0);

#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE

	const uint BufferIndex 		= VertexIndex;
	const uint BlendOffsetCount = {DataInterfaceName}_InputWeightLookupStream[BufferIndex];
	return ComputeBoneMatrixWithUnlimitedInfluences({DataInterfaceName}_BoneMatrices, {DataInterfaceName}_InputWeightStream, {DataInterfaceName}_InputWeightIndexSize, BlendOffsetCount);

#else

	const uint StreamOffset = (VertexIndex * ({DataInterfaceName}_InputWeightStride / 4));
	const uint NumInfluences = {DataInterfaceName}_NumBoneInfluences;
	FGPUSkinIndexAndWeight IndicesAndWeights = (FGPUSkinIndexAndWeight)0;
	if (NumInfluences > 0)
	{
		IndicesAndWeights.BlendIndices  = GetBlendIndices_{DataInterfaceName}(StreamOffset, 0);
		IndicesAndWeights.BlendWeights  = GetBlendWeights_{DataInterfaceName}(StreamOffset, 0);
	}
	if (NumInfluences > 4)
	{
		IndicesAndWeights.BlendIndices2 = GetBlendIndices_{DataInterfaceName}(StreamOffset, 1);
		IndicesAndWeights.BlendWeights2 = GetBlendWeights_{DataInterfaceName}(StreamOffset, 1);
	}
	return ComputeBoneMatrixWithLimitedInfluences({DataInterfaceName}_BoneMatrices, IndicesAndWeights, NumInfluences > 4);
#endif
}

float4 GetBoneQuat_{DataInterfaceName}(uint BoneIndex)
{
	return {DataInterfaceName}_BoneQuats[BoneIndex];
}

float4 GetBoneQuatToBlend_{DataInterfaceName}(uint BoneIndex, float4 BoneQuat0, float Weight)
{
	float4 BoneQuat = GetBoneQuat_{DataInterfaceName}(BoneIndex);
	return dot(BoneQuat, BoneQuat0) >= 0 ? Weight * BoneQuat : -Weight * BoneQuat;
}

float4 ReadWeightedBoneQuat_{DataInterfaceName}(uint VertexIndex)
{
#if !ENABLE_DEFORMER_BONES
	return float4(0, 0, 0, 1);
#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE
	float4 BlendQuat = (float4)0;
	float4 BoneQuat0 = (float4)0;

	uint BufferIndex = VertexIndex;
	uint BlendOffsetCount = {DataInterfaceName}_InputWeightLookupStream[BufferIndex];
	int NumBoneInfluencesLocal = BlendOffsetCount & 0xff;
	int IndicesOffset = BlendOffsetCount >> 8;
	int BoneIndexSize = {DataInterfaceName}_InputWeightIndexSize & 0xff;
	int BoneWeightSize = {DataInterfaceName}_InputWeightIndexSize >> 8;
	int WeightsOffset = IndicesOffset + (BoneIndexSize * NumBoneInfluencesLocal);

	for (int InfluenceIndex = 0; InfluenceIndex < NumBoneInfluencesLocal; InfluenceIndex++)
	{
		int BoneIndexOffset = IndicesOffset + (BoneIndexSize * InfluenceIndex);
		int BoneIndex = {DataInterfaceName}_InputWeightStream[BoneIndexOffset];
		if (BoneIndexSize > 1)
		{
			BoneIndex = {DataInterfaceName}_InputWeightStream[BoneIndexOffset + 1] << 8 | BoneIndex;
		}
		int BoneWeightOffset = WeightsOffset + (BoneWeightSize * InfluenceIndex);
		float BoneWeight = float({ DataInterfaceName }_InputWeightStream[BoneWeightOffset]) / 255.0f;
		if (BoneWeightSize > 1)
		{
			BoneWeight = float({DataInterfaceName}_InputWeightStream[BoneWeightOffset + 1] << 8 | {DataInterfaceName}_InputWeightStream[BoneWeightOffset]) / 65535.0;
		}

		if (InfluenceIndex == 0)
		{
			BoneQuat0 = GetBoneQuat_{DataInterfaceName}(BoneIndex);
			BlendQuat += BoneWeight * BoneQuat0;
		}
		else
		{
			BlendQuat += GetBoneQuatToBlend_{DataInterfaceName}(BoneIndex, BoneQuat0, BoneWeight);
		}
	}
	return BlendQuat;
#else
	float4 BlendQuat = (float4)0;
	float4 BoneQuat0 = (float4)0;

	uint StreamOffset = (VertexIndex * ({DataInterfaceName}_InputWeightStride / 4));

	if ({DataInterfaceName}_NumBoneInfluences > 0)
	{
		int4 BlendIndices = GetBlendIndices_{DataInterfaceName}(StreamOffset, 0);
		float4 BlendWeights = GetBlendWeights_{DataInterfaceName}(StreamOffset, 0);

		BoneQuat0 = GetBoneQuat_{DataInterfaceName}(BlendIndices.x);
		BlendQuat += BlendWeights.x * BoneQuat0;
		BlendQuat += GetBoneQuatToBlend_{DataInterfaceName}(BlendIndices.y, BoneQuat0, BlendWeights.y);
		BlendQuat += GetBoneQuatToBlend_{DataInterfaceName}(BlendIndices.z, BoneQuat0, BlendWeights.z);
		BlendQuat += GetBoneQuatToBlend_{DataInterfaceName}(BlendIndices.w, BoneQuat0, BlendWeights.w);
	}

	// todo[CF]: make this a permutation when permutation support is working again?
	if ({DataInterfaceName}_NumBoneInfluences > 4)
	{
		int4 BlendIndices = GetBlendIndices_{DataInterfaceName}(StreamOffset, 1);
		float4 BlendWeights = GetBlendWeights_{DataInterfaceName}(StreamOffset, 1);

		BlendQuat += GetBoneQuatToBlend_{DataInterfaceName}(BlendIndices.x, BoneQuat0, BlendWeights.x);
		BlendQuat += GetBoneQuatToBlend_{DataInterfaceName}(BlendIndices.y, BoneQuat0, BlendWeights.y);
		BlendQuat += GetBoneQuatToBlend_{DataInterfaceName}(BlendIndices.z, BoneQuat0, BlendWeights.z);
		BlendQuat += GetBoneQuatToBlend_{DataInterfaceName}(BlendIndices.w, BoneQuat0, BlendWeights.w);
	}

	return BlendQuat;
#endif
}
