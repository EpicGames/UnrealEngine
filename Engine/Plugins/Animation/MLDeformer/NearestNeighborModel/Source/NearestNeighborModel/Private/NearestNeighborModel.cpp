// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborModel.h"
#include "Algo/Copy.h"
#include "BoneWeights.h"
#include "Components/ExternalMorphSet.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCache.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/Regex.h"
#include "MeshDescription.h"
#include "Misc/FileHelper.h"
#include "Misc/Optional.h"
#include "MLDeformerComponent.h"
#include "MLDeformerGeomCacheTrainingInputAnim.h"
#include "NearestNeighborModelInputInfo.h"
#include "NearestNeighborModelInstance.h"
#include "NearestNeighborModelVizSettings.h"
#include "NearestNeighborOptimizedNetwork.h"
#include "Rendering/MorphTargetVertexInfoBuffers.h"
#include "Rendering/SkeletalMeshModel.h"
#include "ShaderCore.h"
#include "SkeletalMeshAttributes.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NearestNeighborModel)

#define LOCTEXT_NAMESPACE "UNearestNeighborModel"

namespace UE::NearestNeighborModel
{
	class NEARESTNEIGHBORMODEL_API FNearestNeighborModelModule
		: public IModuleInterface
	{
		void StartupModule()
		{
			FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("NearestNeighborModel"))->GetBaseDir(), TEXT("Shaders"));
			AddShaderSourceDirectoryMapping(TEXT("/Plugin/MLDeformer/NearestNeighborModel"), PluginShaderDir);
		}
	};

	bool bNearestNeighborModelUseOptimizedNetwork = true;
	FAutoConsoleVariableRef CVarNearestNeighborModelUseOptimizedNetwork(
		TEXT("p.NearestNeighborModel.UseOptimizedNetwork"),
		bNearestNeighborModelUseOptimizedNetwork,
		TEXT("Whether to use the optimized network.")
	);

	struct FNearestNeighborModelCustomVersion
	{
		enum Type
		{
			BeforeCustomVersionWasAdded = 0,
			AddTrainedBasis = 1,
			DeprecateNumEpochs = 2,
	
			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};
	
		const static FGuid GUID;
	};
	const FGuid FNearestNeighborModelCustomVersion::GUID = FGuid(0xBA926256, 0x6C7947C4, 0x96D1E4F4, 0xCE985342);
	FCustomVersionRegistration GRegisterNearestNeighborModelCustomVersion(FNearestNeighborModelCustomVersion::GUID, FNearestNeighborModelCustomVersion::LatestVersion, TEXT("NearestNeighborModelVer"));

	namespace Private
	{
		TOptional<TArray<int32>> ParseIntegers(const FString& String)
		{
			if (String.IsEmpty())
			{
				TArray<int32> Empty;
			    return Empty;
			}
			TOptional<TArray<int32>> None;
			static const FRegexPattern AllowedCharsPattern(TEXT("^[-,0-9\\s]+$"));
		
			if (!FRegexMatcher(AllowedCharsPattern, String).FindNext())
			{
			    UE_LOGF(LogNearestNeighborModel, Error, "Input contains invalid characters.");
			    return None;
			}
		
			static const FRegexPattern SingleNumberPattern(TEXT("^\\s*(\\d+)\\s*$"));
			static const FRegexPattern RangePattern(TEXT("^\\s*(\\d+)\\s*-\\s*(\\d+)\\s*$"));
		
			TArray<FString> Segments;
			TArray<int32> Result;
		    String.ParseIntoArray(Segments, TEXT(","), true);
		    for (const FString& Segment : Segments)
		    {
		    	bool bSegmentValid = false;
		
		    	FRegexMatcher SingleNumberMatcher(SingleNumberPattern, Segment);
		    	if (SingleNumberMatcher.FindNext())
		    	{
		    	    const int32 SingleNumber = FCString::Atoi(*SingleNumberMatcher.GetCaptureGroup(1));
		    	    Result.Add(SingleNumber);
		    	    bSegmentValid = true;
		    	}
		    	else
		    	{
		    		FRegexMatcher RangeMatcher(RangePattern, Segment);
		    		if (RangeMatcher.FindNext())
		    		{
		    		    const int32 RangeStart = FCString::Atoi(*RangeMatcher.GetCaptureGroup(1));
		    		    const int32 RangeEnd = FCString::Atoi(*RangeMatcher.GetCaptureGroup(2));
		
		    		    for (int32 i = RangeStart; i <= RangeEnd; ++i)
		    		    {
		    		        Result.Add(i);
		    		    }
		    		    bSegmentValid = true;
		    		}
		    	}
		    	
		    	if (!bSegmentValid)
		    	{
		    	    UE_LOGF(LogNearestNeighborModel, Error, "Invalid format in segment: %ls", *Segment);
					return None;
		    	}
		    }
		
			return Result;
		}

		FString IntegersToFormattedString(const TArray<int32>& Array)
		{
		    if (Array.Num() == 0)
		        return "";
		
		    TArray<int32> SortedArray = Array;
		    SortedArray.Sort();
		
		    FString Result;
		    int32 StartRange = SortedArray[0];
		    int32 EndRange = SortedArray[0];
		
		    for (int32 i = 1; i < SortedArray.Num(); ++i)
		    {
		        if (SortedArray[i] == EndRange + 1)
		        {
		            EndRange = SortedArray[i];
		        }
		        else
		        {
		            if (StartRange == EndRange)
		                Result += FString::Printf(TEXT("%d, "), StartRange);
		            else
		                Result += FString::Printf(TEXT("%d-%d, "), StartRange, EndRange);
		
		            StartRange = EndRange = SortedArray[i];
		        }
		    }
		
		    if (StartRange == EndRange)
		        Result += FString::FromInt(StartRange);
		    else
		        Result += FString::Printf(TEXT("%d-%d"), StartRange, EndRange);
		
		    return Result;
		}

		template<typename SrcT, typename TgtT>
		TArray<TgtT> ConvertArray(TConstArrayView<SrcT> Src)
		{
			TArray<TgtT> Tgt;
			Tgt.SetNum(Src.Num());
			for (int32 Index = 0; Index < Src.Num(); ++Index)
			{
				Tgt[Index] = static_cast<TgtT>(Src[Index]);
			}
			return Tgt;
		}

		FString GetArchitectureString(int32 InputDim, TConstArrayView<int32> HiddenLayerDims, int32 OutputDim)
		{
			FString HiddenLayerString; 
			for (int32 Index = 0; Index < HiddenLayerDims.Num() - 1; ++Index)
			{
				HiddenLayerString += FString::Printf(TEXT("%d, "), HiddenLayerDims[Index]);
			}
			HiddenLayerString += FString::FromInt(HiddenLayerDims.Last());
			return FString::Printf(TEXT("%d [%s] %d"), InputDim, *HiddenLayerString, OutputDim);
		}

		TArray<int32> GetDefaultHiddenLayerDims(int32 InputDim, int32 OutputDim)
		{
			check(InputDim > 0 && OutputDim > 0);
			TArray<int32> HiddenLayerDims;
			if (OutputDim >= InputDim)
			{
				int32 Dim = FMath::RoundUpToPowerOfTwo(InputDim);
				do 
				{
					HiddenLayerDims.Add(Dim);
					Dim *= 2;
				} while (Dim < OutputDim);
			}
			else
			{
				int32 Dim = FMath::RoundUpToPowerOfTwo(OutputDim);
				do 
				{
					HiddenLayerDims.Add(Dim);
					Dim *= 2;
				} while (Dim < InputDim);
				Algo::Reverse(HiddenLayerDims);
			}
			return HiddenLayerDims;
		}

		TOptional<FDateTime> GetTimeStamp(const FString& Path)
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (PlatformFile.FileExists(*Path))
			{
				return PlatformFile.GetTimeStamp(*Path);
			}
			else
			{
				return TOptional<FDateTime>();
			}
		}

		TOptional<FDateTime> GetLatestTimeStamp(const TArray<FString>& Paths)
		{
			TOptional<FDateTime> LatestTimeStamp;
			for (const FString& Path : Paths)
			{
				TOptional<FDateTime> TimeStamp = GetTimeStamp(Path);
				if (TimeStamp.IsSet())
				{
					if (!LatestTimeStamp.IsSet() || TimeStamp.GetValue() > LatestTimeStamp.GetValue())
					{
						LatestTimeStamp = TimeStamp;
					}
				}
			}
			return LatestTimeStamp;
		}

		TArray<int32> GetIncludedFrames(TConstArrayView<int32> ExcludedFrames, int32 NumFrames)
		{
			TArray<int32> IncludedFrames;
			TArray<int32> FrameMap;
			IncludedFrames.Reserve(NumFrames);
			for (int32 Frame = 0; Frame < NumFrames; Frame++)
			{
				if (!ExcludedFrames.Contains(Frame))
				{
					IncludedFrames.Add(Frame);
				}
			}
			return IncludedFrames;
		}

		bool IsArrayChanged(EPropertyChangeType::Type ChangeType)
		{
			return (ChangeType & (EPropertyChangeType::ArrayAdd | EPropertyChangeType::ArrayRemove | EPropertyChangeType::ArrayClear | EPropertyChangeType::ArrayMove)) != 0;
		}

		TOptional<TArray<float>> ReadTxt(const FString& FilePath)
		{
			TOptional<TArray<float>> Empty;

			TArray<FString> Lines;
			if (FFileHelper::LoadFileToStringArray(Lines, *FilePath))
			{
				TArray<float> Result;
				Result.Reserve(Lines.Num());
				for (const FString& Line : Lines)
				{
					float Value;
					if (Line.TrimStartAndEnd().IsNumeric())
					{
						Value = FCString::Atof(*Line);
						Result.Add(Value);
					}
					else
					{
						UE_LOGF(LogNearestNeighborModel, Error, "Invalid value in file: %ls", *Line);
						return Empty;
					}
				}
				return Result;
			}
			else
			{
				UE_LOGF(LogNearestNeighborModel, Error, "Failed to read file: %ls", *FilePath);
				return Empty;
			}
		}

#if WITH_EDITORONLY_DATA
		TArray<FName> GetVertexFloatAttributeNames(const USkeletalMesh& SkelMesh)
		{
			constexpr int32 LODIndex = 0;

			if (!SkelMesh.HasMeshDescription(LODIndex))
			{
				return {};
			}

			const FMeshDescription* MeshDescription = SkelMesh.GetMeshDescription(LODIndex);
			
			const TAttributesSet<FVertexID> AttributesSet = MeshDescription->VertexAttributes();
			TArray<FName> AttributeNames;
			AttributesSet.ForEach([&AttributeNames, &AttributesSet](const FName AttributeName, const auto AttributesRef)
			{
				const bool bIsAutoGenerated = (AttributesRef.GetFlags() & EMeshAttributeFlags::AutoGenerated) != EMeshAttributeFlags::None;
				if (!bIsAutoGenerated && AttributesSet.HasAttributeOfType<float>(AttributeName))
				{
					AttributeNames.Add(AttributeName);
				}
			});
			return AttributeNames;
		}
#endif
	};
}
IMPLEMENT_MODULE(UE::NearestNeighborModel::FNearestNeighborModelModule, NearestNeighborModel)


// ================================== UNearestNeighborModelSection ==================================
#if WITH_EDITOR
const UAnimSequence* UNearestNeighborModelSection::GetNeighborPoses() const
{
	return NeighborPoses;
}

UAnimSequence* UNearestNeighborModelSection::GetMutableNeighborPoses() const
{
	return NeighborPoses;
}

const UGeometryCache* UNearestNeighborModelSection::GetNeighborMeshes() const
{
	return NeighborMeshes;
}

UGeometryCache* UNearestNeighborModelSection::GetMutableNeighborMeshes() const
{
	return NeighborMeshes;
}

void UNearestNeighborModelSection::SetVertexMapString(const FString& InString)
{
	VertexMapString = InString;
	UpdateVertexWeights();
	check(Model);
	Model->InvalidateTrainingModelOnly();
}

int32 UNearestNeighborModelSection::GetAssetNumNeighbors() const
{
	using UE::NearestNeighborModel::FHelpers;
	return FMath::Min(FHelpers::GetNumFrames(GetNeighborPoses()), FHelpers::GetNumFrames(GetNeighborMeshes()));
}

void UNearestNeighborModelSection::SetBasisData(const TArray<float>& InVertexMean, const TArray<float>& InBasis)
{
	const int32 NumBasis = GetNumBasis();
	check(NumBasis > 0);
	check(InVertexMean.Num() == NumVertices * 3);
	check(InBasis.Num() == NumVertices * 3 * NumBasis);
	VertexMean = InVertexMean;
	Basis = InBasis;
	InvalidateInference();
}

void UNearestNeighborModelSection::SetNeighborData(const TArray<float>& InAssetNeighborCoeffs, const TArray<float>& InAssetNeighborOffsets)
{
	const int32 NumBasis = GetNumBasis();
	check(NumBasis > 0);
	const int32 TryNumNeighbors = InAssetNeighborCoeffs.Num() / NumBasis;
	check(TryNumNeighbors * NumBasis == InAssetNeighborCoeffs.Num());
	check(TryNumNeighbors * NumVertices * 3 == InAssetNeighborOffsets.Num());
	AssetNeighborCoeffs = InAssetNeighborCoeffs;
	AssetNeighborOffsets = InAssetNeighborOffsets;
	InvalidateInference();
}

const TArray<int32>&  UNearestNeighborModelSection::GetVertexMap() const
{
	return VertexMap;
}

const TArray<float>& UNearestNeighborModelSection::GetVertexWeights() const
{
	return VertexWeights;
}

const TArray<float>& UNearestNeighborModelSection::GetBasis() const
{
	return Basis;
}

const TArray<float>& UNearestNeighborModelSection::GetVertexMean() const
{
	return VertexMean;
}

const TArray<float>& UNearestNeighborModelSection::GetAssetNeighborCoeffs() const
{
	return AssetNeighborCoeffs;
}

bool UNearestNeighborModelSection::DoesUsePCA() const
{
	return Model && Model->DoesUsePCA();
}

const TArray<float>& UNearestNeighborModelSection::GetAssetNeighborOffsets() const
{
	return AssetNeighborOffsets;
}

const TArray<int32>& UNearestNeighborModelSection::GetExcludedFrames() const
{
	return ExcludedFrames;
}

const TArray<int32>& UNearestNeighborModelSection::GetAssetNeighborIndexMap() const
{
	return AssetNeighborIndexMap;
}

FMLDeformerGeomCacheTrainingInputAnim* UNearestNeighborModelSection::GetInputAnim() const
{
	if (!InputAnim.IsValid())
	{
		InputAnim = MakeUnique<FMLDeformerGeomCacheTrainingInputAnim>();
	}
	InputAnim->SetAnimSequence(NeighborPoses);
	InputAnim->SetGeometryCache(NeighborMeshes);
	return InputAnim.Get();
}

bool UNearestNeighborModelSection::UpdateRuntimeNeighbors()
{
	const int32 NumBasis = GetNumBasis();
	if (NumBasis <= 0)
	{
		return false;
	}
	const int32 AssetNumNeighbors = AssetNeighborCoeffs.Num() / NumBasis;
	if (AssetNumNeighbors * NumBasis != AssetNeighborCoeffs.Num())
	{
		return false;
	}
	const TArray<int32> IncludedFrames = UE::NearestNeighborModel::Private::GetIncludedFrames(ExcludedFrames, AssetNumNeighbors);
	RuntimeNumNeighbors = IncludedFrames.Num();
	if (RuntimeNumNeighbors <= 0)
	{
		return true;
	}
	RuntimeNeighborCoeffs.Reset();
	RuntimeNeighborCoeffs.Reserve(RuntimeNumNeighbors * NumBasis);	
	for (int32 Index = 0; Index < IncludedFrames.Num(); ++Index)
	{
		const int32 Frame = IncludedFrames[Index];
		TConstArrayView<float> AssetCoeffs(AssetNeighborCoeffs.GetData() + Frame * NumBasis,  NumBasis);
		Algo::Copy(AssetCoeffs, RuntimeNeighborCoeffs);
	}
	AssetNeighborIndexMap = IncludedFrames;
	return true;
}

bool UNearestNeighborModelSection::GetRuntimeNeighborOffsets(TArray<float>& OutNeighborOffsets) const
{
	if (!IsReadyForInference())
	{
		return false;
	}
	const int32 AssetNumNeighbors = GetAssetNumNeighbors();
	check(AssetNumNeighbors * NumVertices * 3 == AssetNeighborOffsets.Num());
	OutNeighborOffsets.Reset();
	OutNeighborOffsets.Reserve(RuntimeNumNeighbors * NumVertices * 3);
	const TArray<int32> IncludedFrames = UE::NearestNeighborModel::Private::GetIncludedFrames(ExcludedFrames, AssetNumNeighbors);
	check(IncludedFrames.Num() == RuntimeNumNeighbors);
	for (int32 Index = 0; Index < IncludedFrames.Num(); ++Index)
	{
		const int32 Frame = IncludedFrames[Index];
		TConstArrayView<float> AssetOffsets(AssetNeighborOffsets.GetData() + Frame * NumVertices * 3, NumVertices * 3);
		Algo::Copy(AssetOffsets, OutNeighborOffsets);
	}
	return true;
}
#endif

int32 UNearestNeighborModelSection::GetNumBasis() const
{
	return NumPCACoeffs;
}

int32 UNearestNeighborModelSection::GetNumVertices() const
{
	return NumVertices;
}

int32 UNearestNeighborModelSection::GetRuntimeNumNeighbors() const
{
	return RuntimeNumNeighbors;
}

const TArray<float>& UNearestNeighborModelSection::GetNeighborCoeffs() const
{
	return RuntimeNeighborCoeffs;
}

bool UNearestNeighborModelSection::IsReadyForTraining() const
{
	return bIsReadyForTraining;
}

bool UNearestNeighborModelSection::IsReadyForInference() const
{
	return bIsReadyForInference;
}

#if WITH_EDITOR
UE::NearestNeighborModel::EOpFlag UNearestNeighborModelSection::UpdateForTraining()
{
	if (bIsReadyForTraining)
	{
		return EOpFlag::Success;
	}
	if (!GetModel())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "The parent is not a NearestNeighborModel.");
		return EOpFlag::Error;
	}

	EOpFlag Result = UpdateVertexWeights();
	using namespace UE::NearestNeighborModel;
	if (!OpFlag::HasError(Result))
	{
		bIsReadyForTraining = true;
	}
	return Result;
}

UE::NearestNeighborModel::EOpFlag UNearestNeighborModelSection::UpdateForInference()
{
	if (bIsReadyForInference)
	{
		return EOpFlag::Success;
	}

	EOpFlag Result = EOpFlag::Success;
	if (!IsReadyForTraining())
	{
		Result |= UpdateForTraining();
	}
	using namespace UE::NearestNeighborModel;
	if (OpFlag::HasError(Result))
	{
		return Result;
	}

	if (!IsBasisValid())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "Basis is invalid. Please re-train your model");
		ResetBasisData();
		return EOpFlag::Error;
	}
	if (!UpdateRuntimeNeighbors())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "Failed to update runtime neighbors.");
		return EOpFlag::Error;
	}
	if (!IsNearestNeighborValid())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "Nearest neighbor data is invalid. Please re-train your model to update basis data or re-update your model to update nearest neighbor data.");
		ResetNearestNeighborData();
		return EOpFlag::Error;
	}
	bIsReadyForInference = true;
	return Result;
}

void UNearestNeighborModelSection::InvalidateTraining()
{
	bIsReadyForTraining = bIsReadyForInference = false;
	if (Model)
	{
		Model->InvalidateTrainingModelOnly();
	}
}

void UNearestNeighborModelSection::InvalidateInference()
{
	bIsReadyForInference = false;
	if (Model)
	{
		Model->InvalidateInferenceModelOnly();
	}
}

void UNearestNeighborModelSection::SetModel(UNearestNeighborModel* InModel)
{
	if (Model != InModel)
	{
		Model = InModel;
		Reset();
	}
}

const UNearestNeighborModel* UNearestNeighborModelSection::GetModel() const
{
	return Model;
}

ENearestNeighborModelSectionWeightMapCreationMethod UNearestNeighborModelSection::GetWeightMapCreationMethod() const
{
	return WeightMapCreationMethod;
}

FString UNearestNeighborModelSection::GetBoneNamesString() const
{
	FString Result;
	if (BoneNames.IsEmpty())
	{
		return TEXT("None");
	}
	for (int32 Index = 0; Index < BoneNames.Num() - 1; ++Index)
	{
		Result += BoneNames[Index].ToString() + TEXT(", ");
	}
	Result += BoneNames.Last().ToString();
	return Result;
}

const TArray<FName>& UNearestNeighborModelSection::GetBoneNames() const
{
	return BoneNames;
}

void UNearestNeighborModelSection::SetBoneNames(const TArray<FName>& InBoneNames)
{
	BoneNames = InBoneNames;
	InvalidateTraining();
	UpdateVertexWeights();
}

FString UNearestNeighborModelSection::GetExternalTxtFile() const
{
	return ExternalTxtFile;
}

void UNearestNeighborModelSection::SetExternalTxtFile(const FString& InFile)
{
	ExternalTxtFile = InFile;
	InvalidateTraining();
	UpdateVertexWeights();
}

void UNearestNeighborModelSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FProperty* Property = PropertyChangedEvent.Property;
	if (!Property)
	{
		return;
	}
	if (!Model)
	{
		return;
	}
	if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, NumPCACoeffs))
	{
		Model->UpdateNetworkOutputDim();
		InvalidateTraining();
	}
	if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, WeightMapCreationMethod) ||
		Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, VertexMapString) ||
		Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, AttributeName))
	{
		InvalidateTraining();
		UpdateVertexWeights();
	}
	if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, NeighborPoses))
	{
		if (NeighborPoses)
		{
			InvalidateInference();
		}
	}
	if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, NeighborMeshes))
	{
		if (NeighborMeshes)
		{
			InvalidateInference();
		}
	}
	if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, ExcludedFrames))
	{
		InvalidateInference();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
	Model->PostEditChangeProperty(PropertyChangedEvent);
}

void UNearestNeighborModelSection::SetNumBasis(const int32 InNumBasis)
{
	NumPCACoeffs = InNumBasis;
	InvalidateTraining();
}

int32 UNearestNeighborModelSection::GetMeshIndex() const
{
	return MeshIndex;
}

void UNearestNeighborModelSection::SetMeshIndex(int32 Index)
{
	MeshIndex = Index;
	check(Model);
	Model->InvalidateTrainingModelOnly();
}

bool UNearestNeighborModelSection::IsBasisValid() const
{
	return VertexMean.Num() == NumVertices * 3 && Basis.Num() == NumVertices * 3 * GetNumBasis();
}

bool UNearestNeighborModelSection::IsBasisEmpty() const
{
	return VertexMean.IsEmpty() && Basis.IsEmpty();
}

void UNearestNeighborModelSection::ResetBasisData()
{
	VertexMean.Reset();
	Basis.Reset();
}

bool UNearestNeighborModelSection::IsNearestNeighborValid() const
{
	return IsNearestNeighborEmpty() 
	|| (RuntimeNeighborCoeffs.Num() == RuntimeNumNeighbors * GetNumBasis() && NumVertices > 0);
}

bool UNearestNeighborModelSection::IsNearestNeighborEmpty() const
{
	return RuntimeNumNeighbors == 0 && RuntimeNeighborCoeffs.IsEmpty();
}

void UNearestNeighborModelSection::Reset()
{
	NumVertices = 0;
	RuntimeNumNeighbors = 0;
	VertexMap.Reset();
	ResetBasisData();
	ResetNearestNeighborData();
	bIsReadyForTraining = false;
	bIsReadyForInference = false;
}

void UNearestNeighborModelSection::ResetNearestNeighborData()
{
	AssetNeighborCoeffs.Reset();
	AssetNeighborOffsets.Reset();
	AssetNeighborIndexMap.Reset();
	RuntimeNumNeighbors = 0;
	RuntimeNeighborCoeffs.Reset();
}

UE::NearestNeighborModel::EOpFlag UNearestNeighborModelSection::UpdateVertexWeights()
{
	EOpFlag Result = EOpFlag::Error;
	if (WeightMapCreationMethod == ENearestNeighborModelSectionWeightMapCreationMethod::FromText)
	{
		Result = UpdateVertexWeightsFromText();
	}
	else if (WeightMapCreationMethod == ENearestNeighborModelSectionWeightMapCreationMethod::SelectedBones)
	{
		Result = UpdateVertexWeightsSelectedBones();
	}
	else if (WeightMapCreationMethod == ENearestNeighborModelSectionWeightMapCreationMethod::VertexAttributes)
	{
		Result = UpdateVertexWeightsVertexAttributes();
	}
	else if (WeightMapCreationMethod == ENearestNeighborModelSectionWeightMapCreationMethod::ExternalTxt)
	{
		Result = UpdateVertexWeightsExternalTxt();
	}
	else
	{
		UE_LOGF(LogNearestNeighborModel, Error, "Invalid WeightMapCreationMethod.");
	}
	using namespace UE::NearestNeighborModel;
	if (OpFlag::HasError(Result))
	{
		VertexMap.Reset();
		VertexWeights.Reset();
		return Result;
	}

	NumVertices = VertexMap.Num();
	check(VertexWeights.Num() == NumVertices);

	if (DoesUsePCA())
	{
		if (NumVertices <= GetNumBasis())
		{
			UE_LOGF(LogNearestNeighborModel, Error, "NumVertices %d needs to be larger than NumBasis %d.", NumVertices, GetNumBasis());
			Result |= EOpFlag::Error;
		}
	}
	return Result;
}

UE::NearestNeighborModel::EOpFlag UNearestNeighborModelSection::UpdateVertexWeightsFromText()
{
	using namespace UE::NearestNeighborModel;
	if (!GetModel())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "NearestNeighborModel is invalid.");
		return EOpFlag::Error;
	}
	const USkeletalMesh* const SkeletalMesh = GetModel()->GetSkeletalMesh();
	if (!SkeletalMesh)
	{
		return EOpFlag::Error;
	}
	if (!SkeletalMesh->GetImportedModel())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "SkeletalMesh has no imported model.");
		return EOpFlag::Error;
	}
	constexpr int32 LODIndex = 0;
	if (!SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogNearestNeighborModel, Error, "SkeletalMesh has no LODModel.");
		return EOpFlag::Error;
	}
	const FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];

	using namespace Private;
	TOptional<TArray<int32>> VertexMapOpt = ParseIntegers(VertexMapString);
	if (VertexMapOpt.IsSet())
	{
		VertexMap = MoveTemp(VertexMapOpt.GetValue());
		// TODO: check uniqueness
		if (const int32 MaxIndex = FMath::Max(VertexMap); MaxIndex >= SkeletalMesh->GetNumImportedVertices())
		{
			UE_LOGF(LogNearestNeighborModel, Error, "MaxIndex %d is invalid, SkeletalMesh has %d vertices.", MaxIndex, SkeletalMesh->GetNumImportedVertices());
			VertexMap.Reset();
			return EOpFlag::Error;
		}
		VertexWeights.Init(1, VertexMap.Num());
	}
	else
	{
		UE_LOGF(LogNearestNeighborModel, Error, "VertexMapString is invalid.");
		return EOpFlag::Error;
	}

	return EOpFlag::Success;
}

UE::NearestNeighborModel::EOpFlag UNearestNeighborModelSection::UpdateVertexWeightsSelectedBones()
{
	using namespace UE::NearestNeighborModel;
	if (!GetModel())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "NearestNeighborModel is invalid.");
		return EOpFlag::Error;
	}
	const USkeletalMesh* const SkeletalMesh = GetModel()->GetSkeletalMesh();
	if (!SkeletalMesh)
	{
		return EOpFlag::Error;
	}
	if (!SkeletalMesh->GetImportedModel())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "SkeletalMesh has no imported model.");
		return EOpFlag::Error;
	}
	constexpr int32 LODIndex = 0;
	if (!SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogNearestNeighborModel, Error, "SkeletalMesh has no LODModel.");
		return EOpFlag::Error;
	}
	const FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	TArray<int32> BoneIndices;
	BoneIndices.Reserve(BoneNames.Num());
	for (const FName& BoneName : BoneNames)
	{
		const int32 BoneIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(BoneName);
		if (BoneIndex == INDEX_NONE)
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Bone %ls does not exist in SkeletalMesh.", *BoneName.ToString());
			return EOpFlag::Error;
		}
		BoneIndices.Add(BoneIndex);
	}
	const int32 NumImportedVertices = SkeletalMesh->GetNumImportedVertices();
	TArray<float> Weights; 
	Weights.SetNumZeroed(NumImportedVertices);
	const TArray<FSkelMeshSection>& Sections = LODModel.Sections;
	for (const FSkelMeshSection& Section : Sections)
	{
		if (Section.SoftVertices.Num() != Section.NumVertices)
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Section %d has invalid soft vertices.", Section.MaterialIndex);
			return EOpFlag::Error;
		}
		for (int32 VertexIndex = 0; VertexIndex < Section.NumVertices; ++VertexIndex)
		{
			const FSoftSkinVertex& SoftVertex = Section.SoftVertices[VertexIndex];
			const int32 ImportedIndex = LODModel.MeshToImportVertexMap[Section.BaseVertexIndex + VertexIndex];
			if (Weights[ImportedIndex] > 0) // already processed
			{
				continue;
			}
			for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
			{
				const uint16 WeightByte = SoftVertex.InfluenceWeights[InfluenceIndex];
				if (WeightByte == 0)
				{
					break;
				}
				const int32 BoneIndex = Section.BoneMap[SoftVertex.InfluenceBones[InfluenceIndex]];
				if (BoneIndices.Contains(BoneIndex))
				{
					const float	Weight = static_cast<float>(WeightByte) * UE::AnimationCore::InvMaxRawBoneWeightFloat;
					Weights[ImportedIndex] += Weight;
				}
			}
		}
	}
	VertexMap.Reset();
	VertexMap.Reserve(NumImportedVertices);
	VertexWeights.Reset();
	VertexWeights.Reserve(NumImportedVertices);
	for (int32 VertexIndex = 0; VertexIndex < NumImportedVertices; ++VertexIndex)
	{
		if (Weights[VertexIndex] > 0)
		{
			VertexMap.Add(VertexIndex);
			VertexWeights.Add(Weights[VertexIndex]);
		}
	}
	return EOpFlag::Success;
}

UE::NearestNeighborModel::EOpFlag UNearestNeighborModelSection::UpdateVertexWeightsVertexAttributes()
{
	using namespace UE::NearestNeighborModel;
	if (!GetModel())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "NearestNeighborModel is invalid.");
		return EOpFlag::Error;
	}
	const USkeletalMesh* const SkeletalMesh = GetModel()->GetSkeletalMesh();
	if (!SkeletalMesh)
	{
		return EOpFlag::Error;
	}

	if (!SkeletalMesh->GetImportedModel())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "SkeletalMesh has no imported model.");
		return EOpFlag::Error;
	}
	constexpr int32 LODIndex = 0;
	if (!SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
	{
		UE_LOGF(LogNearestNeighborModel, Error, "SkeletalMesh has no LODModel at LOD 0.");
		return EOpFlag::Error;
	}
	if (!SkeletalMesh->HasMeshDescription(LODIndex))
	{
		UE_LOGF(LogNearestNeighborModel, Error, "SkeletalMesh has no Mesh Description at LOD 0.");
		return EOpFlag::Error;
	}
	const FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];

	const FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LODIndex);
	const TAttributesSet<FVertexID> AttributesSet = MeshDescription->VertexAttributes();
	if (!AttributesSet.HasAttributeOfType<float>(AttributeName))
	{
		UE_LOGF(LogNearestNeighborModel, Error, "There is no float attribute %ls in SkeletalMesh.", *AttributeName.ToString());
		return EOpFlag::Error;
	}
	TVertexAttributesConstRef<float> AttributesRef = AttributesSet.GetAttributesRef<float>(AttributeName);
	
	const int32 NumImportedVertices = SkeletalMesh->GetNumImportedVertices();
	VertexMap.Reset();
	VertexMap.Reserve(NumImportedVertices);
	VertexWeights.Reset();
	VertexWeights.Reserve(NumImportedVertices);
	for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
	{
		const float Value = AttributesRef[VertexID];
		if (Value > 0)
		{
			const int32 Index = LODModel.MeshToImportVertexMap[VertexID.GetValue()];
			VertexMap.Add(VertexID.GetValue());
			VertexWeights.Add(Value);
		}
	}

	return EOpFlag::Success;
}

UE::NearestNeighborModel::EOpFlag UNearestNeighborModelSection::UpdateVertexWeightsExternalTxt()
{
	using namespace UE::NearestNeighborModel;
	if (!GetModel())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "NearestNeighborModel is invalid.");
		return EOpFlag::Error;
	}
	const USkeletalMesh* const SkeletalMesh = GetModel()->GetSkeletalMesh();
	if (!SkeletalMesh)
	{
		return EOpFlag::Error;
	}
	if (!GetModel())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "NearestNeighborModel is invalid.");
		return EOpFlag::Error;
	}

	const int32 NumBaseMeshVerts = Model->GetNumBaseMeshVerts();
	if (NumBaseMeshVerts <= 0)
	{
		UE_LOGF(LogNearestNeighborModel, Error, "SkeletalMesh has no vertices.");
		return EOpFlag::Error;
	}

	using Private::ReadTxt;
	TOptional<TArray<float>> WeightsOpt = ReadTxt(ExternalTxtFile);
	if (!WeightsOpt.IsSet())
	{
		return EOpFlag::Error;
	}
	const TArray<float>& Weights = WeightsOpt.GetValue();
	if (Weights.Num() != NumBaseMeshVerts)
	{
		UE_LOGF(LogNearestNeighborModel, Error, "Number of weights %d does not match number of vertices %d.", Weights.Num(), NumBaseMeshVerts);
		return EOpFlag::Error;
	}
	VertexMap.Reset();
	VertexWeights.Reset();
	for (int32 Index = 0; Index < Weights.Num(); ++Index)
	{
		if (Weights[Index] > 0)
		{
			VertexMap.Add(Index);
			VertexWeights.Add(Weights[Index]);
		}
	}
	return EOpFlag::Success;
}

UE::NearestNeighborModel::EOpFlag UNearestNeighborModelSection::NormalizeVertexWeights()
{
	if (!Model)
	{
		UE_LOGF(LogNearestNeighborModel, Error, "NearestNeighborModel in Section is invalid");
		return EOpFlag::Error;
	}
	const TArray<float>& WeightSum = Model->GetVertexWeightSum();
	if (WeightSum.Num() != Model->GetNumBaseMeshVerts())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "VertexWeightSum is not initialized. Call UNearestNeighborModel::UpdateForTraining() instead.");
		return EOpFlag::Error;
	}
	for (int32 Index = 0; Index < VertexWeights.Num(); ++Index)
	{
		const int32 VertexIndex = VertexMap[Index];
		const float Sum = WeightSum[VertexIndex];
		if (Sum > 1) // Only normalize if Sum > 1
		{
			VertexWeights[Index] /= Sum;
		}
	}
	return EOpFlag::Success;
}

void UNearestNeighborModel::UpdateVersion()
{
	using UE::NearestNeighborModel::FNearestNeighborModelCustomVersion;
	Version = FNearestNeighborModelCustomVersion::LatestVersion;
}

TArray<FName> UNearestNeighborModelSection::GetVertexAttributeNames() const
{
	if (!Model)
	{
		return {};
	}
	const USkeletalMesh* const SkeletalMesh = Model->GetSkeletalMesh();
	if (!SkeletalMesh)
	{
		return {};
	}

	using UE::NearestNeighborModel::Private::GetVertexFloatAttributeNames;
	return GetVertexFloatAttributeNames(*SkeletalMesh);
}
#endif

// ================================== UNearestNeighborModel ==================================
UNearestNeighborModel::UNearestNeighborModel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	SetVizSettings(ObjectInitializer.CreateEditorOnlyDefaultSubobject<UNearestNeighborModelVizSettings>(this, TEXT("VizSettings")));

	// Add two layers of 128 units on default.
	HiddenLayerDims.Add(128);
	HiddenLayerDims.Add(128);
#endif
}

void UNearestNeighborModel::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	#if WITH_EDITORONLY_DATA
		Context.AddTag(FAssetRegistryTag("MLDeformer.NearestNeighborModel.NumIterations", FString::FromInt(NumIterations), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.NearestNeighborModel.BatchSize", FString::FromInt(BatchSize), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.NearestNeighborModel.NumHiddenLayers", FString::FromInt(HiddenLayerDims.Num()), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.NearestNeighborModel.LearningRate", FString::Printf(TEXT("%f"), LearningRate), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.NearestNeighborModel.EarlyStopEpochs", FString::FromInt(EarlyStopEpochs), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.NearestNeighborModel.UseDualQuaternions", bUseDualQuaternionDeltas ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	#endif

	Context.AddTag(FAssetRegistryTag("MLDeformer.NearestNeighborModel.DecayFactor", FString::Printf(TEXT("%f"), DecayFactor), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NearestNeighborModel.NearestNeighborOffsetWeight", FString::Printf(TEXT("%f"), NearestNeighborOffsetWeight), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NearestNeighborModel.UseRBF", bUseRBF ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NearestNeighborModel.RBFSigma", FString::Printf(TEXT("%f"), RBFSigma), FAssetRegistryTag::TT_Numerical));
}

#if WITH_EDITOR
void UNearestNeighborModel::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FProperty* Property = PropertyChangedEvent.Property;
	if (!Property)
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);
		return;
	}
	if (Property->GetFName() == UMLDeformerModel::GetSkeletalMeshPropertyName() ||
		Property->GetFName() == UMLDeformerModel::GetAlignmentTransformPropertyName() ||
		Property->GetFName() == UMLDeformerModel::GetBoneIncludeListPropertyName() ||
		Property->GetFName() == UMLDeformerModel::GetCurveIncludeListPropertyName() ||
		Property->GetFName() == UMLDeformerModel::GetMaxTrainingFramesPropertyName() ||
		Property->GetFName() == UNearestNeighborModel::GetHiddenLayerDimsPropertyName() ||
		Property->GetFName() == UNearestNeighborModel::GetNumIterationsPropertyName() ||
		Property->GetFName() == UNearestNeighborModel::GetBatchSizePropertyName() ||
		Property->GetFName() == UNearestNeighborModel::GetLearningRatePropertyName() ||
		Property->GetFName() == UNearestNeighborModel::GetEarlyStopEpochsPropertyName())
	{
		InvalidateTraining();
	}

	if (Property->GetFName() == UNearestNeighborModel::GetBoneIncludeListPropertyName() ||
		Property->GetFName() == UNearestNeighborModel::GetCurveIncludeListPropertyName())
	{
		UpdateNetworkInputDim();
	}
	if (Property->GetFName() == UNearestNeighborModel::GetSectionsPropertyName())
	{
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
		{
			const int32 NewIndex = PropertyChangedEvent.GetArrayIndex(UNearestNeighborModel::GetSectionsPropertyName().ToString());
			FSection* const Section = OnSectionAdded(NewIndex);
			check(Section);

			FString MapString;
			if (GetSkeletalMesh())
			{
				const TArray<FInt32Range> Ranges = GetMeshVertRanges(*GetSkeletalMesh());
				const int32 MeshIndex = Section->GetMeshIndex(); 
				if (Ranges.IsValidIndex(MeshIndex))
				{
					MapString = FString::Printf(TEXT("%d-%d"), Ranges[MeshIndex].GetLowerBoundValue(), Ranges[MeshIndex].GetUpperBoundValue() - 1);
				}
			}
			Section->SetVertexMapString(MapString);
		}
		if (UE::NearestNeighborModel::Private::IsArrayChanged(PropertyChangedEvent.ChangeType))
		{
			InvalidateTrainingModelOnly();
			UpdateNetworkOutputDim();
		}
	}
	if (Property->GetFName() == UNearestNeighborModel::GetUsePCAPropertyName() ||
		Property->GetFName() == UNearestNeighborModel::GetNumBasisPerSectionPropertyName())
	{
		UpdateSectionNumBasis();
		UpdateNetworkOutputDim();
		InvalidateTraining();
	}
	if (Property->GetFName() == UNearestNeighborModel::GetUseFileCachePropertyName() ||
		Property->GetFName() == UNearestNeighborModel::GetFileCacheDirectoryPropertyName())
	{
		UpdateFileCache();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UNearestNeighborModel::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) 
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	const FProperty* Property = PropertyChangedEvent.Property;
	if (!Property)
	{
		return;
	}
	const FProperty* MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	if (!MemberProperty)
	{
		return;
	}

	const FString MemberPropertyName = MemberProperty->GetName();
	if (MemberPropertyName == TEXT("TrainingInputAnims"))
	{
		bool bInvalidate = true;
		const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(MemberPropertyName);
		const TArray<FMLDeformerGeomCacheTrainingInputAnim> InputAnims = GetTrainingInputAnims();
		if (ArrayIndex >= 0 && ArrayIndex < InputAnims.Num())
		{
			if (Property->GetFName() == FMLDeformerTrainingInputAnim::GetAnimSequencePropertyName() && !InputAnims[ArrayIndex].GetAnimSequence())
			{
				bInvalidate = false;
			}
			if (Property->GetFName() == FMLDeformerGeomCacheTrainingInputAnim::GetGeomCachePropertyName() && !InputAnims[ArrayIndex].GetGeometryCache())
			{
				bInvalidate = false;
			}
		}
		if (bInvalidate)
		{
			InvalidateTraining();
		}
	}
}

void UNearestNeighborModel::FinalizeMorphTargets()
{
	for(FSection* Section : Sections)
	{
		Section->FinalizeMorphTargets();
	}
	Super::FinalizeMorphTargets();
}
#endif

UMLDeformerInputInfo* UNearestNeighborModel::CreateInputInfo()
{
	return NewObject<UNearestNeighborModelInputInfo>(this, NAME_None, RF_Transactional);
}

FString UNearestNeighborModel::GetDisplayName() const
{
	return "Nearest Neighbor Model";
}

bool UNearestNeighborModel::IsTrained() const
{
	return GetOptimizedNetwork() != nullptr;
}

FString UNearestNeighborModel::GetDefaultDeformerGraphAssetPath() const
{
	if (bUseDualQuaternionDeltas)
	{
		return TEXT("/NearestNeighborModel/Deformers/DG_DQ_RecomputeNormals.DG_DQ_RecomputeNormals");
	}
	else
	{
		return TEXT("/DeformerGraph/Deformers/DG_LinearBlendSkin_Morph_Cloth_RecomputeNormals.DG_LinearBlendSkin_Morph_Cloth_RecomputeNormals");
	}
}

UMLDeformerModelInstance* UNearestNeighborModel::CreateModelInstance(UMLDeformerComponent* Component)
{
	return NewObject<UNearestNeighborModelInstance>(Component);
}

void UNearestNeighborModel::Serialize(FArchive& Archive)
{
	Super::Serialize(Archive);
	using UE::NearestNeighborModel::FNearestNeighborModelCustomVersion;
	if (Archive.IsLoading())
	{
		Version = Archive.CustomVer(FNearestNeighborModelCustomVersion::GUID);
	}
	Archive.UsingCustomVersion(FNearestNeighborModelCustomVersion::GUID);
}

void UNearestNeighborModel::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA

	if (IsBeforeTrainedBasisAdded())
	{
		bUsePCA = true;
	}

	if (IsBeforeDeprecateNumEpochs())
	{
		NumIterations = 20000;
	}

	// Update data set at runtime.
	UpdateNetworkInputDim();
	UpdateNetworkOutputDim();

	UpdateFileCache();

	for (FSection* Section : Sections)
	{
		if (Section)
		{
			Section->SetModel(this);
		}
	}
#endif
}

int32 UNearestNeighborModel::GetNumSections() const
{
	return Sections.Num();
}

const UNearestNeighborModelSection* UNearestNeighborModel::GetSectionPtr(int32 Index) const
{
	return Sections[Index];
}

const UNearestNeighborModelSection& UNearestNeighborModel::GetSection(int32 Index) const
{
	return *Sections[Index];
}

UNearestNeighborModelSection& UNearestNeighborModel::GetSection(int32 Index)
{
	return *Sections[Index];
}

const TArray<int32>& UNearestNeighborModel::GetPCACoeffStarts() const
{
	return PCACoeffStarts;
}

int32 UNearestNeighborModel::GetTotalNumBasis() const
{
	if (DoesUsePCA())
	{
		int32 Sum = 0;
		for(const FSection* Section : Sections)
		{
			Sum += Section->GetNumBasis();
		}
		return Sum;
	}
	else
	{
		return NumBasisPerSection;
	}
}

int32 UNearestNeighborModel::GetNumBasisPerSection() const
{
	return NumBasisPerSection;
}

int32 UNearestNeighborModel::GetTotalNumNeighbors() const
{
	int32 Sum = 0;
	for(const FSection* Section : Sections)
	{
		Sum += Section->GetRuntimeNumNeighbors();
	}
	return Sum;
}

float UNearestNeighborModel::GetDecayFactor() const
{
	return DecayFactor;
}

float UNearestNeighborModel::GetNearestNeighborOffsetWeight() const
{
	return NearestNeighborOffsetWeight;
}

void UNearestNeighborModel::ClipInputs(TArrayView<float> Inputs) const
{
	const int32 Num = FMath::Min(Inputs.Num(), FMath::Min(InputsMin.Num(), InputsMax.Num()));
	for(int32 Index = 0; Index < Num; Index++)
	{
		float Max = InputsMax[Index];
		float Min = InputsMin[Index];
		if (bUseInputMultipliers && Index / 3 < InputMultipliers.Num())
		{
			const float Multiplier = InputMultipliers[Index / 3][Index % 3];
			Max *= Multiplier;
			Min *= Multiplier;
		}
		Inputs[Index] = FMath::Clamp(Inputs[Index], Min, Max);
	}
}


TWeakObjectPtr<const UNearestNeighborOptimizedNetwork> UNearestNeighborModel::GetOptimizedNetwork() const
{
	return OptimizedNetwork;
}

TWeakObjectPtr<UNearestNeighborOptimizedNetwork> UNearestNeighborModel::GetOptimizedNetwork()
{
	return OptimizedNetwork;
}

bool UNearestNeighborModel::IsReadyForTraining() const
{
	return bIsReadyForTraining;
}

bool UNearestNeighborModel::IsReadyForInference() const
{
	return bIsReadyForInference;
}

bool UNearestNeighborModel::DoesUseRBF() const
{
	return bUseRBF;
}

float UNearestNeighborModel::GetRBFSigma() const
{
	return RBFSigma;
}


#if WITH_EDITORONLY_DATA
TArray<FInt32Range> UNearestNeighborModel::GetMeshVertRanges(const USkeletalMesh& SkelMesh)
{
	constexpr int32 LODIndex = 0;

	if (!SkelMesh.HasMeshDescription(LODIndex))
	{
		return {};
	}

	const FMeshDescription* MeshDescription = SkelMesh.GetMeshDescription(LODIndex);
	const FSkeletalMeshConstAttributes MeshAttributes(*MeshDescription);

	if (!MeshAttributes.HasSourceGeometryParts())
	{
		return {};
	}

	TArray<FInt32Range> MeshVertRanges;
	MeshVertRanges.Reserve(MeshAttributes.GetNumSourceGeometryParts());
	const FSkeletalMeshAttributesShared::FSourceGeometryPartVertexOffsetAndCountConstRef PartOffsetAndCountRef = MeshAttributes.GetSourceGeometryPartVertexOffsetAndCounts();
	for (const FSourceGeometryPartID GeometryPartID: MeshAttributes.SourceGeometryParts().GetElementIDs())
	{
		TArrayView<const int32> OffsetAndCount = PartOffsetAndCountRef.Get(GeometryPartID);
		MeshVertRanges.Emplace(OffsetAndCount[0], OffsetAndCount[0] + OffsetAndCount[1]);
	}
	return MeshVertRanges;
}
#endif


#if WITH_EDITOR
UNearestNeighborModelSection* UNearestNeighborModel::OnSectionAdded(int32 NewIndex)
{
	if (!Sections.IsValidIndex(NewIndex))
	{
		return nullptr;
	}
	UNearestNeighborModelSection* Section = NewObject<UNearestNeighborModelSection>(this);
	if (!Section)
	{
		return nullptr;
	}
	Section->SetModel(this);
	if (!DoesUsePCA())
	{
		Section->SetNumBasis(NumBasisPerSection);
	}
	Sections[NewIndex] = Section;
	return Section;
}

FDateTime UNearestNeighborModel::GetNetworkLastWriteTime() const
{
	return NetworkLastWriteTime;
}

FString UNearestNeighborModel::GetNetworkLastWriteArchitectureString() const
{
	return NetworkLastWriteArchitectureString;
}

FDateTime UNearestNeighborModel::GetMorphTargetsLastWriteTime() const
{
	return MorphTargetsLastWriteTime;
}

TArray<FString> UNearestNeighborModel::GetCachedDeltasPaths() const
{
	return {FString::Printf(TEXT("%s/deltas.bin"), *GetModelDir())};
}

TArray<FString> UNearestNeighborModel::GetCachedPCAPaths() const
{
	const int32 NumSections = GetNumSections();
	TArray<FString> FilePaths; 
	FilePaths.Reserve(NumSections * 2);
	for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
	{
		const FString VertexMeanPath = FString::Printf(TEXT("%s/vertex_mean_%d.npy"), *GetModelDir(), SectionIndex);
		FilePaths.Add(VertexMeanPath);
		const FString PCABasisPath = FString::Printf(TEXT("%s/pca_basis_%d.npy"), *GetModelDir(), SectionIndex);
		FilePaths.Add(PCABasisPath);
	}
	return FilePaths;
}

TArray<FString> UNearestNeighborModel::GetCachedNetworkPaths() const
{
	return {FString::Printf(TEXT("%s/NearestNeighborModel.ubnne"), *GetModelDir())};
}

TOptional<FDateTime> UNearestNeighborModel::GetCachedDeltasTimestamp() const
{
	return CachedDeltasTimestamp;
}

TOptional<FDateTime> UNearestNeighborModel::GetCachedPCATimestamp() const
{
	return CachedPCATimestamp;
}

TOptional<FDateTime> UNearestNeighborModel::GetCachedNetworkTimestamp() const
{
	return CachedNetworkTimestamp; 
}

FMLDeformerGeomCacheTrainingInputAnim* UNearestNeighborModel::GetNearestNeighborAnim(int32 SectionIndex)
{
	if (FMath::IsWithin<int32>(SectionIndex, 0, Sections.Num()))
	{
		return Sections[SectionIndex]->GetInputAnim();
	}
	return nullptr;
}

const FMLDeformerGeomCacheTrainingInputAnim* UNearestNeighborModel::GetNearestNeighborAnim(int32 SectionIndex) const
{
	if (FMath::IsWithin<int32>(SectionIndex, 0, Sections.Num()))
	{
		return Sections[SectionIndex]->GetInputAnim();
	}
	return nullptr;
}

void UNearestNeighborModel::UpdatePCACoeffStarts()
{
	uint32 Acc = 0;
	PCACoeffStarts.Reset(Sections.Num());
	for(const FSection* Section : Sections)
	{
		PCACoeffStarts.Add(Acc);
		Acc += Section->GetNumBasis();
	}
}

void UNearestNeighborModel::UpdateNetworkInputDim()
{
	InputDim = 3 * GetBoneIncludeList().Num();
}

void UNearestNeighborModel::UpdateNetworkOutputDim()
{
	OutputDim = GetTotalNumBasis();
}

void UNearestNeighborModel::ClearReferences()
{
	for (FSection* Section : Sections)
	{
		if (Section)
		{
			Section->ClearReferences();
		}
	}
	for (FMLDeformerGeomCacheTrainingInputAnim& InputAnim : GetTrainingInputAnims())
	{
		InputAnim.SetAnimSequence(nullptr);
		InputAnim.SetGeometryCache(nullptr);
	}
	UMLDeformerVizSettings* const Viz = GetVizSettings();
	if (Viz)
	{
		Viz->SetTestAnimSequence(nullptr);
	}
}

bool UNearestNeighborModel::IsBeforeCustomVersionWasAdded() const
{
	using UE::NearestNeighborModel::FNearestNeighborModelCustomVersion;
	return Version < FNearestNeighborModelCustomVersion::BeforeCustomVersionWasAdded;
}

bool UNearestNeighborModel::IsBeforeTrainedBasisAdded() const
{
	using UE::NearestNeighborModel::FNearestNeighborModelCustomVersion;
	return Version < FNearestNeighborModelCustomVersion::AddTrainedBasis;
}

bool UNearestNeighborModel::IsBeforeDeprecateNumEpochs() const
{
	using UE::NearestNeighborModel::FNearestNeighborModelCustomVersion;
	return Version < FNearestNeighborModelCustomVersion::DeprecateNumEpochs;
}

UE::NearestNeighborModel::EOpFlag UNearestNeighborModel::CheckHiddenLayerDims()
{
	check(InputDim > 0 && OutputDim > 0);
	if (HiddenLayerDims.IsEmpty())
	{
		HiddenLayerDims = UE::NearestNeighborModel::Private::GetDefaultHiddenLayerDims(InputDim, OutputDim);
		UE_LOGF(LogNearestNeighborModel, Error, "HiddenLayerDims is empty, which will lead to bad results. Default hidden layer dims are added for you. Please double check and train model again.");
		return EOpFlag::Error;
	}
	for (int32 Index = 0; Index < HiddenLayerDims.Num(); ++Index)
	{
		if (HiddenLayerDims[Index] <= 0)
		{
			UE_LOGF(LogNearestNeighborModel, Error, "HiddenLayerDims[%d] is %d. Non-positive numbers will lead to bad results", Index, HiddenLayerDims[Index]);
			return EOpFlag::Error;
		}
	}
	return EOpFlag::Success;
}

void UNearestNeighborModel::UpdateInputMultipliers()
{
	if (bUseInputMultipliers)
	{
		const int32 NumMultipliers = InputMultipliers.Num();
		const int32 NumBones = GetBoneIncludeList().Num();
		InputMultipliers.SetNum(NumBones);
		for (int32 Index = NumMultipliers; Index < NumBones; ++Index)
		{
			InputMultipliers[Index] = FVector3f(1.0f, 1.0f, 1.0f);
		}
	}
}

void UNearestNeighborModel::UpdateFileCache()
{
	UpdateCachedDeltasTimestamp();
	UpdateCachedPCATimestamp();
	UpdateCachedNetworkTimestamp();
}

const FString& UNearestNeighborModel::GetFileCacheDirectory() const
{
	return FileCacheDirectory;
}

void UNearestNeighborModel::SetFileCacheDirectory(const FString& InFileCacheDirectory)
{
	FileCacheDirectory = InFileCacheDirectory;
}

void UNearestNeighborModel::UpdateSectionNumBasis()
{
	for (FSection* Section : Sections)
	{
		if (Section)
		{
			Section->SetNumBasis(NumBasisPerSection);
		}
	}
}

void UNearestNeighborModel::UpdateCachedDeltasTimestamp()
{
	CachedDeltasTimestamp = UE::NearestNeighborModel::Private::GetLatestTimeStamp(GetCachedDeltasPaths());
}

void UNearestNeighborModel::UpdateCachedPCATimestamp()
{
	CachedPCATimestamp = UE::NearestNeighborModel::Private::GetLatestTimeStamp(GetCachedPCAPaths());
}

void UNearestNeighborModel::UpdateCachedNetworkTimestamp()
{
	CachedNetworkTimestamp = UE::NearestNeighborModel::Private::GetLatestTimeStamp(GetCachedNetworkPaths());
}

void UNearestNeighborModel::UpdateMorphTargetsLastWriteTime()
{
	MorphTargetsLastWriteTime = FDateTime::UtcNow().GetTicks();
}

const TArray<float>& UNearestNeighborModel::GetVertexWeightSum() const
{
	return VertexWeightSum;
}

void UNearestNeighborModel::NormalizeVertexWeights()
{
	if (GetNumBaseMeshVerts() == 0)
	{
		return;
	}

	VertexWeightSum.Init(0.f, GetNumBaseMeshVerts());
	for (const FSection* Section : Sections)
	{
		check(Section);
		const TArray<int32>& SectionVertexMap = Section->GetVertexMap();
		const TArray<float>& SectionVertexWeights = Section->GetVertexWeights();
		for (int32 Index = 0; Index < SectionVertexMap.Num(); ++Index)
		{
			VertexWeightSum[SectionVertexMap[Index]] += SectionVertexWeights[Index];
		}
	}
	for (FSection* Section : Sections)
	{
		check(Section);
		Section->NormalizeVertexWeights();
	}
	VertexWeightSum.Reset();
}

void UNearestNeighborModelSection::ClearReferences()
{
	NeighborPoses = nullptr;
	NeighborMeshes = nullptr;
}

void UNearestNeighborModelSection::FinalizeMorphTargets()
{
	VertexMean.Empty();
	Basis.Empty();
	AssetNeighborCoeffs.Empty();
	AssetNeighborOffsets.Empty();
	AssetNeighborIndexMap.Empty();
}

UE::NearestNeighborModel::EOpFlag UNearestNeighborModel::UpdateForTraining()
{
	if (IsReadyForTraining())
	{
		return EOpFlag::Success;
	}
	using namespace UE::NearestNeighborModel;
	EOpFlag Result = EOpFlag::Success;
	if (DoesUsePCA())
	{
		UpdatePCACoeffStarts();
	}
	UpdateNetworkInputDim();
	if (InputDim == 0)
	{
		UE_LOGF(LogNearestNeighborModel, Error, "InputDim is 0. Please add bones to the include list.");
		return EOpFlag::Error;
	}
	UpdateNetworkOutputDim();
	if (OutputDim == 0)
	{
		UE_LOGF(LogNearestNeighborModel, Error, "OutputDim is 0. Please create at least one section.");
		return EOpFlag::Error;
	}
	Result |= CheckHiddenLayerDims();
	if (OpFlag::HasError(Result))
	{
		return Result;
	}

	if (GetNumSections() == 0)
	{
		UE_LOGF(LogNearestNeighborModel, Error, "At least one section is required. Please create a section using the '+' button.");
		return EOpFlag::Error;
	}
	for (FSection* Section : Sections)
	{
		Section->SetModel(this);
		Result |= Section->UpdateForTraining();
	}
	NormalizeVertexWeights();
	VertexWeightSum.Reset();

	if (!OpFlag::HasError(Result))
	{
		bIsReadyForTraining = true;
	}
	return Result;
}

void UNearestNeighborModel::InvalidateTraining()
{
	InvalidateTrainingModelOnly();
	for (FSection* Section : Sections)
	{
		if (Section)
		{
			Section->InvalidateTraining();
		}
	}
}

void UNearestNeighborModel::InvalidateTrainingModelOnly()
{
	bIsReadyForTraining = bIsReadyForInference = false;
}

UE::NearestNeighborModel::EOpFlag UNearestNeighborModel::UpdateForInference()
{
	if (IsReadyForInference())
	{
		return EOpFlag::Success;
	}
	using namespace UE::NearestNeighborModel;
	if (!bIsReadyForTraining)
	{
		return EOpFlag::Error;
	}

	EOpFlag Result = EOpFlag::Success;
	for (FSection* Section : Sections)
	{
		Result |= Section->UpdateForInference();
		if (OpFlag::HasError(Result))
		{
			return Result;
		}
	}
	UpdateInputMultipliers();
	bIsReadyForInference = true;
	return Result;
}

void UNearestNeighborModel::InvalidateInference()
{
	InvalidateInferenceModelOnly();
	for (FSection* Section : Sections)
	{
		Section->InvalidateInference();
	}
}

void UNearestNeighborModel::InvalidateInferenceModelOnly()
{
	bIsReadyForInference = false;
}

FString UNearestNeighborModel::GetModelDir() const
{
	if (bUseFileCache)
	{
		return FileCacheDirectory;
	}
	else
	{
		return FPaths::ProjectIntermediateDir() + TEXT("NearestNeighborModel/");
	}
}

void UNearestNeighborModel::SetOptimizedNetwork(UNearestNeighborOptimizedNetwork* InOptimizedNetwork)
{
	OptimizedNetwork = InOptimizedNetwork;
	GetReinitModelInstanceDelegate().Broadcast();
}

void UNearestNeighborModel::ClearOptimizedNetwork()
{
	if (OptimizedNetwork)
	{
		OptimizedNetwork->ConditionalBeginDestroy();
		OptimizedNetwork = nullptr;
		NetworkLastWriteTime = 0;
		NetworkLastWriteArchitectureString = FString();
		InvalidateInference();
	}
}

bool UNearestNeighborModel::LoadOptimizedNetworkFromFile(const FString& Filename)
{
	const FString FullPath = FPaths::ConvertRelativePathToFull(Filename);
	if (FPaths::FileExists(FullPath))
	{
		UE_LOGF(LogNearestNeighborModel, Display, "Loading Network file '%ls'...", *FullPath);

		// When we create a new UNearestNeighborOptimizedNetwork object we need to set the input
		// and output sizes as it cannot get them from the model on load.
		UNearestNeighborOptimizedNetwork* Result = NewObject<UNearestNeighborOptimizedNetwork>(this);
		Result->SetNumInputs(InputDim);
		Result->SetNumOutputs(OutputDim);
		const bool bSuccess = Result->Load(FullPath);
		
		if (bSuccess)
		{
			ClearOptimizedNetwork();
			SetOptimizedNetwork(Result);
			NetworkLastWriteTime = FDateTime::UtcNow().GetTicks();
			using UE::NearestNeighborModel::Private::GetArchitectureString;
			NetworkLastWriteArchitectureString = GetArchitectureString(InputDim, HiddenLayerDims, OutputDim);
			InvalidateInference();
			UpdateVersion();
			return true;
		}
		else
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Failed to load Network file '%ls'!", *FullPath);
			Result->ConditionalBeginDestroy();
		}
	}
	else
	{
		UE_LOGF(LogNearestNeighborModel, Error, "Network file '%ls' does not exist!", *FullPath);
	}
	return false;
}

void UNearestNeighborModel::RemoveAllSections()
{
	for (TObjectPtr<UNearestNeighborModelSection>& Section : Sections)
	{
		Section->ConditionalBeginDestroy();
	}
	Sections.Empty();
}
#endif	// WITH_EDITOR

int32 UNearestNeighborModel::GetNumNetworkOutputs() const
{
	return OptimizedNetwork ? OptimizedNetwork->GetNumOutputs() : 0;
}

#undef LOCTEXT_NAMESPACE
