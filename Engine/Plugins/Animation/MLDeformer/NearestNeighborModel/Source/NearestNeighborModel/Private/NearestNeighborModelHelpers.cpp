// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborModelHelpers.h"

#include "Animation/AnimSequence.h"
#include "GeometryCache.h"

NEARESTNEIGHBORMODEL_API DEFINE_LOG_CATEGORY(LogNearestNeighborModel)

namespace UE::NearestNeighborModel
{
#if WITH_EDITOR
	int32 FHelpers::GetNumFrames(const UAnimSequence* Anim)
	{
		if (Anim)
		{
			if (const IAnimationDataModel* DataModel = Anim->GetDataModel())
			{
				return DataModel->GetNumberOfKeys();
			}
		}
		return 0;
	}
	
	int32 FHelpers::GetNumFrames(const UGeometryCache* Cache)
	{
		return Cache ? Cache->GetEndFrame() - Cache->GetStartFrame() + 1 : 0;
	}
#endif

	namespace OpFlag
	{
		bool HasError(EOpFlag Flag)
		{
			return (Flag & EOpFlag::Error) != EOpFlag::Success;
		}
		
		EOpFlag AddError(EOpFlag Flag)
		{
			return Flag | EOpFlag::Error;
		}

		bool HasWarning(EOpFlag Flag)
		{
			return (Flag & EOpFlag::Warning) != EOpFlag::Success;
		}

		EOpFlag AddWarning(EOpFlag Flag)
		{
			return Flag | EOpFlag::Warning;
		}
	};

	EOpFlag operator|(EOpFlag A, EOpFlag B)
	{
		return static_cast<EOpFlag>(static_cast<uint8>(A) | static_cast<uint8>(B));
	}

	EOpFlag& operator|=(EOpFlag& A, EOpFlag B)
	{
		A = A | B;
		return A;
	}

	EOpFlag operator&(EOpFlag A, EOpFlag B)
	{
		return static_cast<EOpFlag>(static_cast<uint8>(A) & static_cast<uint8>(B));
	}

	EOpFlag& operator&=(EOpFlag& A, EOpFlag B)
	{
		A = A & B;
		return A;
	}

	EOpFlag ToOpFlag(int32 Value)
	{
		check(Value >= 0 && Value <= 2)
		return static_cast<EOpFlag>(Value);
	}
};