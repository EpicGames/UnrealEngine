// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborModelInputInfo.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NearestNeighborModelInputInfo)

void UNearestNeighborModelInputInfo::ExtractBoneRotations(USkeletalMeshComponent* SkelMeshComponent, TArray<float>& OutRotations) const
{
    const int32 NumBones = GetNumBones();
    const int32 NumFloats = NumBones * 3;
    OutRotations.Reset(NumFloats);
    OutRotations.AddUninitialized(NumFloats);

    const TArray<FTransform>& BoneTransforms = SkelMeshComponent->GetBoneSpaceTransforms();

    // Extract BoneRotations from BoneTransforms
    TArray<FQuat> BoneRotations;
    BoneRotations.AddZeroed(NumBones);
    for (int32 Index = 0; Index < NumBones; ++Index)
    {
        const FName BoneName = GetBoneName(Index);
        const int32 SkelMeshBoneIndex = SkelMeshComponent->GetBoneIndex(BoneName);
        FVector RotationVector = FVector::Zero();
        if(SkelMeshBoneIndex != INDEX_NONE)
        {
            BoneRotations[Index] = BoneTransforms[SkelMeshBoneIndex].GetRotation();
        }
    }

    ComputeNetworkInput(BoneRotations, OutRotations);
}

int32 UNearestNeighborModelInputInfo::CalcNumNeuralNetInputs(int32 NumFloatsPerBone, int32 NumFloatsPerCurve) const
{
    return
        BoneNames.Num() * NumFloatsPerBone +
        CurveNames.Num() * NumFloatsPerCurve;
}

void UNearestNeighborModelInputInfo::InitRefBoneRotations(const USkeletalMesh* SkelMesh)
{
    RefBoneRotations.Reset();
    if (SkelMesh)
    {
        const FReferenceSkeleton& RefSkel = SkelMesh->GetRefSkeleton();
        const TArray<FTransform>& RefBonePose = RefSkel.GetRefBonePose();
        const int32 NumBones = GetNumBones();
        RefBoneRotations.SetNumUninitialized(NumBones);
        for(int32 Index = 0; Index < NumBones; Index++)
        {
            const FName BoneName = GetBoneName(Index);
            const int32 BoneIndex = RefSkel.FindBoneIndex(BoneName);
            RefBoneRotations[Index] = RefBonePose[BoneIndex].GetRotation();
        }
    }
}

void UNearestNeighborModelInputInfo::ComputeNetworkInput(TConstArrayView<FQuat> BoneRotations, TArrayView<float> OutputView) const
{
    const int32 NumBones = GetNumBones();
    check(BoneRotations.Num() == NumBones && RefBoneRotations.Num() == NumBones);
    check(OutputView.Num() >= NumBones * 3);

    int32 Index = 0;
    for(int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
    {
        FQuat RestR = RefBoneRotations[BoneIndex];
        FQuat LocalR = BoneRotations[BoneIndex];

        const FQuat R =  RestR.Inverse() * LocalR;
        const float Scalar = R.W >= 0 ? 2.0f : -2.0f;

        OutputView[Index++] = R.X * Scalar;
        OutputView[Index++] = R.Y * Scalar;
        OutputView[Index++] = R.Z * Scalar;
    }
}
