// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborModelInstance.h"

#include "Algo/MinElement.h"
#include "Components/ExternalMorphSet.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "NearestNeighborModel.h"
#include "NearestNeighborModelInputInfo.h"
#include "NearestNeighborOptimizedNetwork.h"
#include "MLDeformerAsset.h"
#include "MLDeformerComponent.h"
#include "Components/ExternalMorphSet.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NearestNeighborModelInstance)
CSV_DECLARE_CATEGORY_MODULE_EXTERN(MLDEFORMERFRAMEWORK_API, MLDeformer);

void UNearestNeighborModelInstance::Init(USkeletalMeshComponent* SkelMeshComponent)
{
	UMLDeformerMorphModelInstance::Init(SkelMeshComponent);
	InitInstanceData();
	const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
	if (!NearestNeighborModel)
	{
		return;
	}
	if (NearestNeighborModel->GetOptimizedNetwork().IsValid())
	{
		InitOptimizedNetworkInstance();
		check(OptimizedNetworkInstance);
	}
}

void UNearestNeighborModelInstance::Execute(float ModelWeight)
{
	CSV_SCOPED_TIMING_STAT(MLDeformer, NearestNeighborExecute);
	check(OptimizedNetworkInstance);
	OptimizedNetworkInstance->Run();
}

bool UNearestNeighborModelInstance::SetupInputs()
{
	const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
	if (!NearestNeighborModel || !NearestNeighborModel->GetInputInfo())
	{
		return false;
	}
	if (!NearestNeighborModel->IsReadyForInference())
	{
		return false;
	}
	if (SkeletalMeshComponent == nullptr ||
		SkeletalMeshComponent->GetSkeletalMeshAsset() == nullptr ||
		!bIsCompatible)
	{
		return false;
	}
	UConstNetworkPtr OptimizedNetwork = NearestNeighborModel->GetOptimizedNetwork();
	if (!OptimizedNetwork.IsValid() || !OptimizedNetworkInstance)
	{
		return false;
	}

	TOptional<TArrayView<float>> OptionalInputView = GetInputView();
	if (!OptionalInputView.IsSet())
	{
		return false;
	}
	TArrayView<float> InputView = OptionalInputView.GetValue();
	const int32 NumNeuralNetInputs = InputView.Num();

	// If the neural network expects a different number of inputs, do nothing.
	const int32 NumFloatsPerBone = NearestNeighborModel->GetNumFloatsPerBone();
	const int32 NumFloatsPerCurve = NearestNeighborModel->GetNumFloatsPerCurve();
	const int64 NumDeformerAssetInputs = NearestNeighborModel->GetInputInfo()->CalcNumNeuralNetInputs(NumFloatsPerBone, NumFloatsPerCurve);
	if (NumNeuralNetInputs != NumDeformerAssetInputs)
	{
		return false;
	}

	const int64 NumFloatsWritten = SetNeuralNetworkInputValues(InputView.GetData(), NumNeuralNetInputs);
	check(NumFloatsWritten == NumNeuralNetInputs);
	// Clip network inputs based on training set min and max values.
	NearestNeighborModel->ClipInputs(InputView);
	return true;
}

void UNearestNeighborModelInstance::Tick(float DeltaTime, float ModelWeight)
{
	// Post init hasn't yet succeeded, try again.
	// This could for example happen when you add an ML Deformer component, but your SkeletalMesh isn't setup yet, but later becomes valid.
	if (Model && !HasPostInitialized())
	{
		PostMLDeformerComponentInit();
	}

	bool bExecuteCalled = false;
	if (ModelWeight > 0.0001f && HasValidTransforms() && SetupInputs())
	{
		// Execute the model instance.
		// For models using neural networks this will perform the inference, 
		// calculate the network outputs and possibly use them, depending on how the model works.
		Execute(ModelWeight);
		RunNearestNeighborModel(DeltaTime, ModelWeight);

		const bool bDrawDebugInputPoses = UE::MLDeformer::CVarMLDeformerDebugDrawInputPoses->GetBool();
		if (bDrawDebugInputPoses)
		{
			DrawDebugInputPoses();
		}
		
		bExecuteCalled = true;
	}
	else
	{
		HandleZeroModelWeight();
	}

	// Do some things afterwards, such as copying over debug actor data.
	PostTick(bExecuteCalled);
}

int64 UNearestNeighborModelInstance::SetBoneTransforms(float* OutputBuffer, int64 OutputBufferSize, int64 StartIndex)
{
	// Get the transforms for the bones we used during training.
	// These are in the space relative to their parent.
	UpdateBoneTransforms();

	check(Model);
    const UNearestNeighborModelInputInfo* InputInfo = static_cast<UNearestNeighborModelInputInfo*>(Model->GetInputInfo());
	check(InputInfo);
    const int32 NumBones = InputInfo->GetNumBones();

    // Extract BoneRotations from TrainingBoneTransforms (@todo: optimize)
    TArray<FQuat> BoneRotations; 
	BoneRotations.SetNumUninitialized(NumBones);
	const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
	check(NearestNeighborModel);
	const int32 NumFloatsPerBone = NearestNeighborModel->GetNumFloatsPerBone();
    const int64 EndIndex = StartIndex + NumBones * NumFloatsPerBone;
	check(EndIndex <= OutputBufferSize); // Make sure we don't write past the OutputBuffer.
    
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
        BoneRotations[BoneIndex] = TrainingBoneTransforms[BoneIndex].GetRotation();
	}

    // Write the transforms into the output buffer.
    InputInfo->ComputeNetworkInput(BoneRotations, TArrayView<float>(OutputBuffer + StartIndex, NumBones * NumFloatsPerBone));

	return EndIndex;
}

void UNearestNeighborModelInstance::Reset()
{
	bNeedsReset = true;
}

FString UNearestNeighborModelInstance::CheckCompatibility(USkeletalMeshComponent* InSkelMeshComponent, bool LogIssues)
{
	ErrorText = FString();

	// If we're not compatible, generate a compatibility string.
	USkeletalMesh* const SkelMesh = InSkelMeshComponent ? InSkelMeshComponent->GetSkeletalMeshAsset() : nullptr;
	UMLDeformerInputInfo* const InputInfo = Model->GetInputInfo();
	if (SkelMesh && !InputInfo->IsCompatible(InSkelMeshComponent) && Model->GetDeformerAsset())
	{
		ErrorText += InputInfo->GenerateCompatibilityErrorString(InSkelMeshComponent);
		ErrorText += "\n";
		if (LogIssues)
		{
			UE_LOGF(LogNearestNeighborModel, Error, "ML Deformer '%ls' isn't compatible with Skeletal Mesh '%ls'.\nReason(s):\n%ls", 
				*Model->GetDeformerAsset()->GetName(), 
				*SkelMesh->GetName(), 
				*ErrorText);
		}
	}

	const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
	if (!NearestNeighborModel)
	{
		return FString();
	}
	// Verify the number of inputs versus the expected number of inputs.
	const int32 NumFloatsPerBone = NearestNeighborModel->GetNumFloatsPerBone();
	const int32 NumFloatsPerCurve = NearestNeighborModel->GetNumFloatsPerCurve();
	const int64 NumDeformerAssetInputs = static_cast<int64>(Model->GetInputInfo()->CalcNumNeuralNetInputs(NumFloatsPerBone, NumFloatsPerCurve));
	int64 NumNeuralNetInputs = -1;
	UConstNetworkPtr OptimizedNetwork = NearestNeighborModel->GetOptimizedNetwork();
	if (OptimizedNetwork.IsValid())
	{
		NumNeuralNetInputs = OptimizedNetwork->GetNumInputs();
	}

	// Only check compatibility after the network is loaded. 
	if (NumNeuralNetInputs >= 0 && NumNeuralNetInputs != NumDeformerAssetInputs) 
	{
		const FString InputErrorString = "The number of network inputs doesn't match the asset. Please retrain the asset."; 
		ErrorText += InputErrorString + "\n";
		if (LogIssues && Model->GetDeformerAsset())
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Deformer '%ls': %ls", *(Model->GetDeformerAsset()->GetName()), *InputErrorString);
		}
	}

	return ErrorText;
}

void UNearestNeighborModelInstance::InitInstanceData(int32 NumMorphWeights)
{
	if (NumMorphWeights == INDEX_NONE)
	{
		const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return;
		}
		NumMorphWeights = 1 + NearestNeighborModel->GetTotalNumBasis() + NearestNeighborModel->GetTotalNumNeighbors();
	}
	DistanceBuffer.SetNumZeroed(NumMorphWeights);
}

UNearestNeighborModel* UNearestNeighborModelInstance::GetCastModel() const
{
	return Cast<UNearestNeighborModel>(Model);
}

#if WITH_EDITOR
TArray<uint32> UNearestNeighborModelInstance::GetNearestNeighborIds() const
{
	return NearestNeighborIds;
}
#endif

void UNearestNeighborModelInstance::InitOptimizedNetworkInstance()
{
	const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
	if (!NearestNeighborModel)
	{
		return;
	}
	UConstNetworkPtr Network = NearestNeighborModel->GetOptimizedNetwork();
	OptimizedNetworkInstance = Network.IsValid() ? Network->CreateInstance(this) : nullptr;
}

namespace UE::NearestNeighborModel::Private
{
	constexpr float FastExpMinusOne(float X)
	{
		return X + X * X / 2.0f + X * X * X / 6.0f;
	}

	float GetDecayCoeff(float DeltaTime, float DecayFactor)
	{
		/** DecayCoeff = (e ^ {(DecayFactor - 1) * DeltaTime + Delta0} - 1) / (e ^ {Delta0} - 1)
		 * DecayFactor is a user defined value. DecayCoeff is the actual coefficient used to compute decay. 
		 * DecayCoeff is an exponential decay of DeltaTime.
		 * When DecayFactor = 1, DecayCoeff = 1; when DecayFactor = 0 && DeltaTime = Delta0, DecayCoeff = 0
		 */	
		constexpr float Delta0 = 1.0f / 30.f;
		const float Delta = (DecayFactor - 1.0f) * DeltaTime;
		return FastExpMinusOne(FMath::Max(Delta + Delta0, 0.0f)) / FastExpMinusOne(Delta0);
	}

	float ComputeDistanceSquared(TConstArrayView<float> V1, TConstArrayView<float> V2)
	{
		check(V1.Num() == V2.Num());
		float DistanceSquared = 0;
		for (int32 Index = 0; Index < V1.Num(); ++Index)
		{
			const float D = V1[Index] - V2[Index];
			DistanceSquared += D * D;
		}
		return DistanceSquared;
	}

	void ComputeNeighborDistances(TArrayView<float> OutDistances, const UNearestNeighborModel::FSection& Section, TConstArrayView<float> Coeffs)
	{
		const int32 NumBasis = Section.GetNumBasis();
		const int32 NumNeighbors = Section.GetRuntimeNumNeighbors();
		check(OutDistances.Num() == NumNeighbors);
		TConstArrayView<float> NeighborCoeffs = Section.GetNeighborCoeffs();
		for (int32 NeighborId = 0; NeighborId < NumNeighbors; ++NeighborId)
		{
			TConstArrayView<float> SingleCoeffs(NeighborCoeffs.GetData() + NeighborId * NumBasis, NumBasis);
			OutDistances[NeighborId] = ComputeDistanceSquared(Coeffs, SingleCoeffs);
		}
	}

	void UpdateWeightWithDecay(float& OutDecayedWeight, float& InOutPrevWeight, float NewWeight, float DecayCoeff, bool bReset)
	{
		if (bReset)
		{
			OutDecayedWeight = NewWeight;
			InOutPrevWeight = NewWeight;
		}
		else
		{
			const float NewW = (1.0f - DecayCoeff) * NewWeight + DecayCoeff * InOutPrevWeight;
			OutDecayedWeight = NewW;
			InOutPrevWeight = NewW;
		}
	}

	void UpdateNearestNeighborWeights(TArrayView<float> OutMorphWeights, TArrayView<float> InOutPrevWeights, TConstArrayView<float> SquaredDistances, float OffsetWeight, float DecayCoeff, bool bReset)
	{
		const int32 NumNeighbors = OutMorphWeights.Num();
		check(InOutPrevWeights.Num() == NumNeighbors);
		check(SquaredDistances.Num() == NumNeighbors);
		if (NumNeighbors <= 0)
		{
			return;
		}

		const float* MinD2 = Algo::MinElement(SquaredDistances);
		const int32 MinIndex = MinD2 - SquaredDistances.GetData();

		for (int32 Index = 0; Index < NumNeighbors; ++Index)
		{
			const float W = Index == MinIndex ? OffsetWeight : 0.0f;
			UpdateWeightWithDecay(OutMorphWeights[Index], InOutPrevWeights[Index], W, DecayCoeff, bReset);
		}
	}

	// SquaredDistances buffer will be overwritten after this function.
	void UpdateRBFWeights(TArrayView<float> OutMorphWeights, TArrayView<float> InOutPrevWeights, TArrayView<float> SquaredDistances, float Sigma, float OffsetWeight, float DecayCoeff, bool bReset)
	{
		const int32 NumNeighbors = OutMorphWeights.Num();
		check(InOutPrevWeights.Num() == NumNeighbors);
		check(SquaredDistances.Num() == NumNeighbors);
		check(NumNeighbors > 0);

		const float MinD2 = *Algo::MinElement(SquaredDistances);
		float SumWeights = 0.0f;
		const float Sigma2 = FMath::Max(Sigma * Sigma, 1e-6f);
		TArrayView<float> Weights = SquaredDistances;
		for (int32 Index = 0; Index < NumNeighbors; ++Index)
		{
			float Weight = (SquaredDistances[Index] - MinD2) / Sigma2;
			constexpr float CutOff = 3.0f;
			if (Weight < CutOff)
			{
				Weight = FMath::Exp(-Weight);
				Weights[Index] = Weight;
				SumWeights += Weight;
			}
			else
			{
				Weights[Index] = 0.0f;
			}
		}

		SumWeights = FMath::Max(SumWeights, 1e-6f);
		for (int32 Index = 0; Index < NumNeighbors; ++Index)
		{
			const float W = Weights[Index] / SumWeights * OffsetWeight;
			UpdateWeightWithDecay(OutMorphWeights[Index], InOutPrevWeights[Index], W, DecayCoeff, bReset);
		}
	}
}

void UNearestNeighborModelInstance::RunNearestNeighborModel(float DeltaTime, float ModelWeight)
{
	const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
	if (!NearestNeighborModel)
	{
		return;
	}
	using UE::NearestNeighborModel::Private::GetDecayCoeff;
	const float DecayCoeff = GetDecayCoeff(DeltaTime, NearestNeighborModel->GetDecayFactor());

	// Grab the weight data for this morph set.
	// This could potentially fail if we are applying this deformer to the wrong skeletal mesh component.
	const int32 LOD = SkeletalMeshComponent->GetPredictedLODLevel();
	FExternalMorphSetWeights* const WeightData = FindWeightData(LOD);
	if (WeightData == nullptr)
	{
		return;
	}

#if WITH_EDITOR
	NearestNeighborIds.SetNum(NearestNeighborModel->GetNumSections());
#endif

	TOptional<TArrayView<float>> OptionalOutputView = GetOutputView();
	if (!OptionalOutputView.IsSet())
	{
		WeightData->ZeroWeights();
		return;
	}
	TArrayView<float> OutputView = OptionalOutputView.GetValue();

	const int32 NumNetworkWeights = OutputView.Num();
	const int32 NumMorphTargets = WeightData->Weights.Num();
	const int32 TotalNumNeighbors = NearestNeighborModel->GetTotalNumNeighbors();

	if (NumMorphTargets == NumNetworkWeights + 1 + TotalNumNeighbors)
	{
		InitInstanceData(NumMorphTargets);
		bNeedsReset = PreviousWeights.Num() != NumMorphTargets;
		if (bNeedsReset)
		{
			PreviousWeights.SetNumZeroed(NumMorphTargets);
		}

		const float OffsetWeight = NearestNeighborModel->GetNearestNeighborOffsetWeight();
		WeightData->Weights[0] = ModelWeight;

		// Update all generated morph target weights with the values calculated by our neural network.
		for (int32 MorphIndex = 0; MorphIndex < NumNetworkWeights; ++MorphIndex)
		{
			const float W = OutputView[MorphIndex] * ModelWeight;
			using UE::NearestNeighborModel::Private::UpdateWeightWithDecay;
			WeightData->Weights[MorphIndex + 1] = W;
		}

		int32 NeighborOffset = NumNetworkWeights + 1;
		for (int32 SectionIndex = 0; SectionIndex < NearestNeighborModel->GetNumSections(); SectionIndex++)
		{
			const UNearestNeighborModel::FSection& Section = NearestNeighborModel->GetSection(SectionIndex);
			if (!Section.IsReadyForInference())
			{
				continue;
			}

			int32 CoeffStart = 0;
			if (NearestNeighborModel->DoesUsePCA())
			{
				TConstArrayView<int32> PCACoeffStarts = NearestNeighborModel->GetPCACoeffStarts();
				if (!PCACoeffStarts.IsValidIndex(SectionIndex))
				{
					continue;
				}
				CoeffStart = PCACoeffStarts[SectionIndex];
			}
			else
			{
				CoeffStart = 0;
			}
			const int32 NumCoeffs = Section.GetNumBasis();
			TConstArrayView<float> Coeffs(OutputView.GetData() + CoeffStart, NumCoeffs);

			const int32 NumNeighbors = Section.GetRuntimeNumNeighbors();
			if (NumNeighbors <= 0)
			{
				continue;
			}

			TArrayView<float> SquaredDistances(DistanceBuffer.GetData() + NeighborOffset, NumNeighbors);
			using UE::NearestNeighborModel::Private::ComputeNeighborDistances;
			ComputeNeighborDistances(SquaredDistances, Section, Coeffs);
#if WITH_EDITOR
			const int32 NeighborId = NumNeighbors > 0 ? (Algo::MinElement(SquaredDistances) - SquaredDistances.GetData()) : INDEX_NONE;
			NearestNeighborIds[SectionIndex] = NeighborId;
#endif // WITH_EDITOR

			TArrayView<float> SectionMorphWeights(WeightData->Weights.GetData() + NeighborOffset, NumNeighbors);
			TArrayView<float> SectionPreviousWeights(PreviousWeights.GetData() + NeighborOffset, NumNeighbors);
			if (NearestNeighborModel->DoesUseRBF())
			{
				const float Sigma = NearestNeighborModel->GetRBFSigma();
				using UE::NearestNeighborModel::Private::UpdateRBFWeights;
				UpdateRBFWeights(SectionMorphWeights, SectionPreviousWeights, SquaredDistances, Sigma, OffsetWeight * ModelWeight, DecayCoeff, bNeedsReset);
			}
			else
			{
				using UE::NearestNeighborModel::Private::UpdateNearestNeighborWeights;
				UpdateNearestNeighborWeights(SectionMorphWeights, SectionPreviousWeights, SquaredDistances, OffsetWeight * ModelWeight, DecayCoeff, bNeedsReset);
			}

			NeighborOffset += NumNeighbors;
		}

		if (bNeedsReset)
		{
			bNeedsReset = false;
		}
	}
}

TOptional<TArrayView<float>> UNearestNeighborModelInstance::GetInputView() const
{
	const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
	check(NearestNeighborModel);

	UConstNetworkPtr OptimizedNetwork = NearestNeighborModel->GetOptimizedNetwork();
	if (OptimizedNetwork.IsValid() && OptimizedNetworkInstance)
	{
		float* InputData = OptimizedNetworkInstance->GetInputs().GetData();
		const int32 NumInputs = OptimizedNetwork->GetNumInputs();
		return TArrayView<float>(InputData, NumInputs);
	}
	return TOptional<TArrayView<float>>();
}

TOptional<TArrayView<float>> UNearestNeighborModelInstance::GetOutputView() const
{
	const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
	check(NearestNeighborModel);

	UConstNetworkPtr OptimizedNetwork = NearestNeighborModel->GetOptimizedNetwork();
	if (OptimizedNetwork.IsValid() && OptimizedNetworkInstance)
	{
		float* OutputData = OptimizedNetworkInstance->GetOutputs().GetData();
		const int32 NumOutputs = OptimizedNetwork->GetNumOutputs();
		return TArrayView<float>(OutputData, NumOutputs);
	}
	return TOptional<TArrayView<float>>();
}

TArray<float> UNearestNeighborModelInstance::Eval(const TArray<float>& InputData)
{
	const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
	check(NearestNeighborModel);
	TArray<float> Empty;
	TOptional<TArrayView<float>> OptionalInputView = GetInputView();
	if (!OptionalInputView.IsSet())
	{
		return Empty;
	}
	TArrayView<float> InputView = OptionalInputView.GetValue();
	const int32 NumNeuralNetInputs = InputView.Num();

	if (InputData.Num() != NumNeuralNetInputs)
	{
		return Empty;
	}

	for (int32 i = 0; i < NumNeuralNetInputs; ++i)
	{
		InputView[i] = InputData[i];
	}
	NearestNeighborModel->ClipInputs(InputView);

	check(OptimizedNetworkInstance);
	OptimizedNetworkInstance->Run();
	TArrayView<float> Outputs = OptimizedNetworkInstance->GetOutputs();
	return TArray<float>(Outputs.GetData(), Outputs.Num());
}