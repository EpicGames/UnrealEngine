// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborOptimizedNetwork.h"

#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"

#include "NNE.h"
#include "NNERuntime.h"
#include "NNERuntimeCPU.h"
#include "NNEModelData.h"
#include "NNERuntimeBasicCpuBuilder.h"

//--------------------------------------------------------------------------
// UNearestNeighborOptimizedNetwork
//--------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(NearestNeighborOptimizedNetwork)

namespace UE::NearestNeighborModel::Private
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	/** Creates the FileData from the legacy network format and clears it. */
	static inline void CreateFileDataAndClearLayers(TArray<uint8>& OutFileData, TArray<TObjectPtr<UNearestNeighborNetworkLayer>>& Layers)
	{
		UE::NNE::RuntimeBasic::FModelBuilder Builder;

		TArray<UE::NNE::RuntimeBasic::FModelBuilderElement, TInlineAllocator<32>> LayerElements;
		LayerElements.Reserve(2 * Layers.Num());

		for (UNearestNeighborNetworkLayer* Layer : Layers)
		{
			if (UNearestNeighborNetworkLayer_Gemm_Prelu* GemmPreluLayer = Cast<UNearestNeighborNetworkLayer_Gemm_Prelu>(Layer))
			{
				LayerElements.Add(Builder.MakeLinear(
					GemmPreluLayer->NumInputs,
					GemmPreluLayer->NumOutputs,
					GemmPreluLayer->Parameters[0].Values,
					GemmPreluLayer->Parameters[1].Values));

				LayerElements.Add(Builder.MakePReLU(GemmPreluLayer->NumOutputs, Builder.MakeValuesConstant(GemmPreluLayer->NumOutputs, GemmPreluLayer->Parameters[2].Values[0])));
			}
			else if (UNearestNeighborNetworkLayer_Gemm* GemmLayer = Cast<UNearestNeighborNetworkLayer_Gemm>(Layer))
			{
				LayerElements.Add(Builder.MakeLinear(
					GemmLayer->NumInputs,
					GemmLayer->NumOutputs,
					GemmLayer->Parameters[0].Values,
					GemmLayer->Parameters[1].Values));
			}
			else
			{
				checkf(false, TEXT("Unknown Layer Type"));
			}
		}

		uint32 InputSize, OutputSize;
		Builder.WriteFileDataAndReset(OutFileData, InputSize, OutputSize, Builder.MakeSequence(LayerElements));

		Layers.Empty();
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

const TCHAR* UNearestNeighborOptimizedNetwork::RuntimeName = TEXT("NNERuntimeBasicCpu");
const TCHAR* UNearestNeighborOptimizedNetwork::RuntimeModuleName = TEXT("NNERuntimeBasicCpu");


void UNearestNeighborOptimizedNetwork::PostLoad()
{
	Super::PostLoad();

	if (Layers_DEPRECATED.Num() > 0)
	{
		// NumInputs and NumOutputs are not used in the legacy format so we need to load them here.
		NumInputs = Layers_DEPRECATED[0]->NumInputs;
		NumOutputs = Layers_DEPRECATED.Last()->NumOutputs;

		TArray<uint8> FileData;
		UE::NearestNeighborModel::Private::CreateFileDataAndClearLayers(FileData, Layers_DEPRECATED);

		if (!ModelData)
		{
			ModelData = NewObject<UNNEModelData>(this);
		}

		ModelData->Init(TEXT("ubnne"), FileData);

		// The ModelData object stores a copy of the FileData so we manually clear this copy to avoid 
		// having multiple copies of the network in memory at once during loading.
		FileData.Empty();
	}

	// Create in-memory representation of network

	ensureMsgf(FModuleManager::Get().LoadModule(RuntimeModuleName), TEXT("Unable to load runtime module."));

	TWeakInterfacePtr<INNERuntimeCPU> RuntimeCPU = UE::NNE::GetRuntime<INNERuntimeCPU>(RuntimeName);

	if (ensureMsgf(RuntimeCPU.IsValid(), TEXT("Could not find requested NNE Runtime")))
	{
		if (ModelData)
		{
			Model = RuntimeCPU->CreateModelCPU(ModelData);
		}
	}

	// If we are not in the editor then we clear the FileData and FileType since these will be
	// using additional memory if we are loading from the legacy format.

#if !WITH_EDITOR
	ModelData->ClearFileDataAndFileType();
#endif
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UNearestNeighborOptimizedNetwork::Empty()
{
	if (ModelData)
	{
		ModelData->ConditionalBeginDestroy();
		ModelData = nullptr;
		Model.Reset();
	}
}

bool UNearestNeighborOptimizedNetwork::IsEmpty() const
{
	return Model == nullptr;
}

bool UNearestNeighborOptimizedNetwork::Load(const FString& Filename)
{
	Empty();

	TArray<uint8> FileData;
	if (FFileHelper::LoadFileToArray(FileData, *Filename))
	{
		if (!ModelData)
		{
			ModelData = NewObject<UNNEModelData>(this);
		}

		ModelData->Init(TEXT("ubnne"), FileData);

		// Clear FileData to avoid multiple copies in memory at once
		FileData.Empty();

		ensureMsgf(FModuleManager::Get().LoadModule(RuntimeModuleName), TEXT("Unable to load runtime module."));

		TWeakInterfacePtr<INNERuntimeCPU> RuntimeCPU = UE::NNE::GetRuntime<INNERuntimeCPU>(RuntimeName);

		if (ensureMsgf(RuntimeCPU.IsValid(), TEXT("Could not find requested NNE Runtime")))
		{
			if (ModelData)
			{
				Model = RuntimeCPU->CreateModelCPU(ModelData);
			}
		}
	}
	else
	{
		return false;
	}

	return true;
}

int32 UNearestNeighborOptimizedNetwork::GetNumInputs() const
{
	return NumInputs;
}

int32 UNearestNeighborOptimizedNetwork::GetNumOutputs() const
{
	return NumOutputs;
}

void UNearestNeighborOptimizedNetwork::SetNumInputs(int32 InNumInputs)
{
	NumInputs = InNumInputs;
}

void UNearestNeighborOptimizedNetwork::SetNumOutputs(int32 InNumOutputs)
{
	NumOutputs = InNumOutputs;
}

UNearestNeighborOptimizedNetworkInstance* UNearestNeighborOptimizedNetwork::CreateInstance(UObject* Parent) const
{
	UNearestNeighborOptimizedNetworkInstance* Instance = NewObject<UNearestNeighborOptimizedNetworkInstance>(Parent);
	Instance->Init(this);
	return Instance;
}

UE::NNE::IModelCPU* UNearestNeighborOptimizedNetwork::GetModel() const
{
	return Model.Get();
}

//--------------------------------------------------------------------------
// UNearestNeighborOptimizedNetworkInstance
//--------------------------------------------------------------------------

UNearestNeighborOptimizedNetworkInstance::UNearestNeighborOptimizedNetworkInstance() = default;
UNearestNeighborOptimizedNetworkInstance::UNearestNeighborOptimizedNetworkInstance(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {}
UNearestNeighborOptimizedNetworkInstance::UNearestNeighborOptimizedNetworkInstance(FVTableHelper& Helper) : Super(Helper) {}
UNearestNeighborOptimizedNetworkInstance::~UNearestNeighborOptimizedNetworkInstance() = default;

TArrayView<float> UNearestNeighborOptimizedNetworkInstance::GetInputs()
{ 
	return TArrayView<float>(Inputs.GetData(), Inputs.Num());
}

TArrayView<const float> UNearestNeighborOptimizedNetworkInstance::GetInputs() const
{ 
	return TArrayView<const float>(Inputs.GetData(), Inputs.Num());
}

TArrayView<float> UNearestNeighborOptimizedNetworkInstance::GetOutputs()
{ 
	return TArrayView<float>(Outputs.GetData(), Outputs.Num());
}

TArrayView<const float> UNearestNeighborOptimizedNetworkInstance::GetOutputs() const
{ 
	return TArrayView<const float>(Outputs.GetData(), Outputs.Num());
}

const UNearestNeighborOptimizedNetwork* UNearestNeighborOptimizedNetworkInstance::GetNeuralNetwork() const
{ 
	return Network.Get();
}

void UNearestNeighborOptimizedNetworkInstance::Init(const UNearestNeighborOptimizedNetwork* InNeuralNetwork)
{
	Network = InNeuralNetwork;
	Inputs.SetNumZeroed(Network->GetNumInputs());
	Outputs.SetNumZeroed(Network->GetNumOutputs());

	if (Network->GetModel())
	{
		Instance = Network->GetModel()->CreateModelInstanceCPU();
		Instance->SetInputTensorShapes({ UE::NNE::FTensorShape::Make({ 1, (uint32)Inputs.Num() }) });
	}
}

void UNearestNeighborOptimizedNetworkInstance::Run()
{	
	TRACE_CPUPROFILER_EVENT_SCOPE(UNearestNeighborOptimizedNetwork::Run);

	if (Instance)
	{
		Instance->RunSync(
			{ { (void*)Inputs.GetData(), Inputs.Num() * sizeof(float) } },
			{ { (void*)Outputs.GetData(), Outputs.Num() * sizeof(float) } });
	}
}
