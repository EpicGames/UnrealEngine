// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "NearestNeighborOptimizedNetwork.generated.h"

namespace UE::NNE
{
	class IModelCPU;
	class IModelInstanceCPU;
}

class UNNEModelData;

class UNearestNeighborOptimizedNetworkInstance;

USTRUCT()
struct UE_DEPRECATED(5.4, "Nearest Neighbor Network Parameter no longer used for inference.") FNearestNeighborNetworkParameter
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<float> Values;

	UPROPERTY()
	TArray<int32> Shape;
};

UCLASS()
class UE_DEPRECATED(5.4, "Nearest Neighbor Network Layer no longer used for inference.") UNearestNeighborNetworkLayer
	: public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MLDeformer")
	int32 NumInputs = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MLDeformer")
	int32 NumOutputs = 0;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	TArray<FNearestNeighborNetworkParameter> Parameters;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS

UCLASS()
class UE_DEPRECATED(5.4, "Nearest Neighbor Network Layer no longer used for inference.") UNearestNeighborNetworkLayer_Gemm_Prelu : public UNearestNeighborNetworkLayer
{
	GENERATED_BODY()
};

UCLASS()
class UE_DEPRECATED(5.4, "Nearest Neighbor Network Layer no longer used for inference.") UNearestNeighborNetworkLayer_Gemm : public UNearestNeighborNetworkLayer
{
	GENERATED_BODY()
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**
 * The specialized neural network for the MLDeformerModel.
 * This class is used to do inference at runtime at a higher performance than using UNeuralNetwork.
 */
UCLASS()
class UNearestNeighborOptimizedNetwork
	: public UObject
{
	GENERATED_BODY()

public:

	virtual void PostLoad() override;

	/** Clear the network, getting rid of all layers. */
	UFUNCTION(BlueprintCallable, Category = "MLDeformer")
	virtual void Empty();

	/** 
	 * Check if the network is empty or not.
	 * If it is empty, it means it hasn't been loaded, and cannot do anything.
	 * Empty means it has no inputs, outputs or weights.
	 * @return Returns true if empty, otherwise false is returned.
	 */
	virtual bool IsEmpty() const;

	/**
	 * Load the network from a file on disk.
	 * When loading fails, the network will be emptied.
	 * @param Filename The file path to load from.
	 * @return Returns true when successfully loaded this model, otherwise false is returned.
	 */
	virtual bool Load(const FString& Filename);

	/**
	 * Create an instance of this neural network.
	 * @return A pointer to the neural network instance.
	 */
	virtual UNearestNeighborOptimizedNetworkInstance* CreateInstance(UObject* Parent = nullptr) const;


	/**
	 * Get the number of inputs, which is the number of floats the network takes as input.
	 * @result The number of input floats to the network.
	 */
	virtual int32 GetNumInputs() const;

	/**
	 * Get the number of outputs, which is the number of floats the network will output.
	 * @return The number of floats that the network outputs.
	 */
	virtual int32 GetNumOutputs() const;

	/**
	 * Sets the number of inputs, which is the number of floats the network takes as input.
	 * @param InNumInputs The number of input floats to the network.
	 */
	virtual void SetNumInputs(int32 InNumInputs);

	/**
	 * Sets the number of outputs, which is the number of floats the network will output.
	 * @param InNumOutputs The number of floats that the network outputs.
	 */
	virtual void SetNumOutputs(int32 InNumOutputs);

	/**
	 * Get the Model.
	 * @return A pointer to the Model.
	 */
	UE::NNE::IModelCPU* GetModel() const;

protected:

	/** The Model Data used for network */
	UPROPERTY()
	TObjectPtr<UNNEModelData> ModelData;

	/** The Neural Network used. */
	TSharedPtr<UE::NNE::IModelCPU> Model;

	UPROPERTY()
	int32 NumInputs = 0;

	UPROPERTY()
	int32 NumOutputs = 0;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.4, "Layers Object no longer available.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Network format changed."))
	TArray<TObjectPtr<UNearestNeighborNetworkLayer>> Layers_DEPRECATED;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:

	static const TCHAR* RuntimeName;
	static const TCHAR* RuntimeModuleName;
};

/** 
 * An instance of a UNearestNeighborOptimizedNetwork.
 * The instance holds its own input and output buffers and only reads from the network object it was instanced from.
 * This allows it to be multithreaded.
 */
UCLASS()
class UNearestNeighborOptimizedNetworkInstance
	: public UObject
{
	friend class UNearestNeighborOptimizedNetwork;

	GENERATED_BODY()

public:

	// These are required because of the use of TUniquePtr
	UNearestNeighborOptimizedNetworkInstance();
	UNearestNeighborOptimizedNetworkInstance(const FObjectInitializer& ObjectInitializer);
	UNearestNeighborOptimizedNetworkInstance(FVTableHelper& Helper);
	virtual ~UNearestNeighborOptimizedNetworkInstance();

	/**
	 * Get the network input buffer.
	 * @return The array view of floats that represents the network inputs.
	 */
	TArrayView<float> GetInputs();
	TArrayView<const float> GetInputs() const;

	/**
	 * Get the network output buffer.
	 * @return The array view of floats that represents the network outputs.
	 */
	TArrayView<float> GetOutputs();
	TArrayView<const float> GetOutputs() const;

	/**
	 * Get the neural network this is an instance of.
	 * @return A reference to the neural network.
	 */
	const UNearestNeighborOptimizedNetwork* GetNeuralNetwork() const;

	/**
	 * Run the neural network, performing inference.
	 * This will update the values in the output buffer that you can get with the GetOutputs method.
	 * This also assumes you have set all the right input values already.
	 */
	virtual void Run();

protected:

	/**
 	 * Initialize this instance using a given network object.
	 * This is automatically called by the UNearestNeighborOptimizedNetwork::CreateInstance() method.
	 * @param InNeuralNetwork The network that this is an instance of.
	 */
	void Init(const UNearestNeighborOptimizedNetwork* InNeuralNetwork);

protected:
	/** The input values. */
	TArray<float> Inputs;

	/** The output values. */
	TArray<float> Outputs;

	/** The instance data for this Network Instance. */
	TSharedPtr<UE::NNE::IModelInstanceCPU> Instance;

	/** The neural network this is an instance of. */
	UPROPERTY(Transient)
	TObjectPtr<const UNearestNeighborOptimizedNetwork> Network;
};
