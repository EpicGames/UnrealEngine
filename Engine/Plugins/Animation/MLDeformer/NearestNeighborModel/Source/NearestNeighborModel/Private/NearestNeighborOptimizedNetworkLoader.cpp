// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborOptimizedNetworkLoader.h"
#include "NearestNeighborOptimizedNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NearestNeighborOptimizedNetworkLoader)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UNearestNeighborOptimizedNetworkLoader::SetOptimizedNetwork(UNearestNeighborOptimizedNetwork* InNetwork)
{
	Network = InNetwork;
}

UNearestNeighborOptimizedNetwork* UNearestNeighborOptimizedNetworkLoader::GetOptimizedNetwork() const
{
	return Network;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
