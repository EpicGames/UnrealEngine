// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceSkeletonWithQuats.h"

#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusDataDomain.h"
#include "RenderGraphBuilder.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceSkeletonWithQuats)

FString UOptimusSkeletonWithQuatsDataInterface::GetDisplayName() const
{
	return TEXT("SkeletonWithQuats");
}

TArray<FOptimusCDIPinDefinition> UOptimusSkeletonWithQuatsDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({"NumBones", "ReadNumBones", Optimus::DomainName::Vertex, "ReadNumVertices"});
	Defs.Add({"BoneMatrix", "ReadBoneMatrix", {{Optimus::DomainName::Vertex, "ReadNumVertices"}, {Optimus::DomainName::Bone, "ReadNumBones"}}});
	Defs.Add({"BoneWeight", "ReadBoneWeight", {{Optimus::DomainName::Vertex, "ReadNumVertices"}, {Optimus::DomainName::Bone, "ReadNumBones"}}});
	Defs.Add({"WeightedBoneMatrix", "ReadWeightedBoneMatrix", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({"WeightedBoneQuat", "ReadWeightedBoneQuat", Optimus::DomainName::Vertex, "ReadNumVertices" });
	return Defs;
}


TSubclassOf<UActorComponent> UOptimusSkeletonWithQuatsDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}


void UOptimusSkeletonWithQuatsDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumBones"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadBoneMatrix"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadBoneWeight"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadWeightedBoneMatrix"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadWeightedBoneQuat"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint);	
}

BEGIN_SHADER_PARAMETER_STRUCT(FSkeletonWithQuatsDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(uint32, NumBoneInfluences)
	SHADER_PARAMETER(uint32, InputWeightStride)
	SHADER_PARAMETER(uint32, InputWeightIndexSize)
	SHADER_PARAMETER_COMPRESSED_BONE_TRANSFORM_SRV(BoneMatrices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, BoneQuats)
	SHADER_PARAMETER_SRV(Buffer<uint>, InputWeightStream)
	SHADER_PARAMETER_SRV(Buffer<uint>, InputWeightLookupStream)
END_SHADER_PARAMETER_STRUCT()

void UOptimusSkeletonWithQuatsDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FSkeletonWithQuatsDataInterfaceParameters>(UID);
}

void UOptimusSkeletonWithQuatsDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	// Need to be able to support these permutations according to the skeletal mesh settings.
	// todo[CF]: I think GPUSKIN_UNLIMITED_BONE_INFLUENCE and GPUSKIN_BONE_INDEX_UINT16/GPUSKIN_BONE_WEIGHTS_UINT16 are mutually exclusive. So we could save permutations here.
	OutPermutationVector.AddPermutation(TEXT("ENABLE_DEFORMER_BONES"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_UNLIMITED_BONE_INFLUENCE"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_BONE_INDEX_UINT16"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_BONE_WEIGHTS_UINT16"), 2);
}

TCHAR const* UOptimusSkeletonWithQuatsDataInterface::TemplateFilePath = TEXT("/Plugin/MLDeformer/NearestNeighborModel/Private/DataInterfaceSkeletonWithQuats.ush");

TCHAR const* UOptimusSkeletonWithQuatsDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusSkeletonWithQuatsDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusSkeletonWithQuatsDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusSkeletonWithQuatsDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusSkeletonWithQuatsDataProvider* Provider = NewObject<UOptimusSkeletonWithQuatsDataProvider>();
	Provider->SkinnedMesh = Cast<USkinnedMeshComponent>(InBinding);
	return Provider;
}


FComputeDataProviderRenderProxy* UOptimusSkeletonWithQuatsDataProvider::GetRenderProxy()
{
	return new FOptimusSkeletonWithQuatsDataProviderProxy(SkinnedMesh);
}

FOptimusSkeletonWithQuatsDataProviderProxy::FOptimusSkeletonWithQuatsDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent)
{
	if (SkinnedMeshComponent && SkinnedMeshComponent->IsRegistered())
	{
		SceneInterface = SkinnedMeshComponent->GetScene();
		ComponentId = SkinnedMeshComponent->GetPrimitiveSceneId();
	}
	BoneRevisionNumber = SkinnedMeshComponent != nullptr ? SkinnedMeshComponent->GetBoneTransformRevisionNumber() : 0;
}

bool FOptimusSkeletonWithQuatsDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}
	if (SkeletalMeshObject->GetSkeletalMeshRenderData().LODRenderData[SkeletalMeshObject->GetLOD()].RenderSections.Num() != InValidationData.NumInvocations)
	{
		return false;
	}

	return true;
}

struct FSkeletonWithQuatsDataInterfacePermutationIds
{
	uint32 EnableDeformerBones = 0;
	uint32 UnlimitedBoneInfluence = 0;
	uint32 BoneIndexUint16 = 0;
	uint32 BoneWeightsUint16 = 0;

	FSkeletonWithQuatsDataInterfacePermutationIds(FComputeKernelPermutationVector const& PermutationVector)
	{
		{
			static FString Name(TEXT("ENABLE_DEFORMER_BONES"));
			static uint32 Hash = GetTypeHash(Name);
			EnableDeformerBones = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_UNLIMITED_BONE_INFLUENCE"));
			static uint32 Hash = GetTypeHash(Name);
			UnlimitedBoneInfluence = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_BONE_INDEX_UINT16"));
			static uint32 Hash = GetTypeHash(Name);
			BoneIndexUint16 = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_BONE_WEIGHTS_UINT16"));
			static uint32 Hash = GetTypeHash(Name);
			BoneWeightsUint16 = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
	}
};

void FOptimusSkeletonWithQuatsDataProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FSkeletonWithQuatsDataInterfacePermutationIds PermutationIds(InOutPermutationData.PermutationVector);
	for (int32 InvocationIndex = 0; InvocationIndex < InOutPermutationData.NumInvocations; ++InvocationIndex)
	{
		FSkelMeshRenderSection const& RenderSection = LodRenderData->RenderSections[InvocationIndex];

		const bool bPreviousFrame = false;
		FRHIShaderResourceView* BoneBufferSRV = FSkeletalMeshDeformerHelpers::GetBoneBufferForReading(SkeletalMeshObject, LodIndex, InvocationIndex, bPreviousFrame);

		FSkinWeightVertexBuffer const* WeightBuffer = LodRenderData->GetSkinWeightVertexBuffer();
		check(WeightBuffer != nullptr);
		FRHIShaderResourceView* SkinWeightBufferSRV = WeightBuffer->GetDataVertexBuffer()->GetSRV();
		const bool bUnlimitedBoneInfluences = WeightBuffer->GetBoneInfluenceType() == GPUSkinBoneInfluenceType::UnlimitedBoneInfluence;
		FRHIShaderResourceView* InputWeightLookupStreamSRV = bUnlimitedBoneInfluences ? WeightBuffer->GetLookupVertexBuffer()->GetSRV() : nullptr;
		const bool bValidBones = (BoneBufferSRV != nullptr) && (SkinWeightBufferSRV != nullptr) && (!bUnlimitedBoneInfluences || InputWeightLookupStreamSRV != nullptr);
		const bool bUse16BitBoneIndex = WeightBuffer->Use16BitBoneIndex();
		const bool bUse16BitBoneWeights = WeightBuffer->Use16BitBoneWeight();

		InOutPermutationData.PermutationIds[InvocationIndex] |= (bValidBones ? PermutationIds.EnableDeformerBones : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUnlimitedBoneInfluences ? PermutationIds.UnlimitedBoneInfluence : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUse16BitBoneIndex ? PermutationIds.BoneIndexUint16 : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUse16BitBoneWeights ? PermutationIds.BoneWeightsUint16 : 0);
	}
}

void FOptimusSkeletonWithQuatsDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	TArray<FQuat4f> RefToLocalQuats;
	CacheRefToLocalQuats(RefToLocalQuats);

	if (RefToLocalQuats.IsEmpty())
	{
		return;
	}

	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];
	const int32 NumSections = LodRenderData->RenderSections.Num();

	const bool bUnifiedBoneMap = SkeletalMeshRenderData.HasUnifiedBoneMap();
	const int32 NumUniqueSections = bUnifiedBoneMap ? 1 : NumSections;

	PerSectionRefToLocalQuats.SetNum(NumUniqueSections);
	PerSectionRefToLocalQuatsSRVs.SetNum(NumUniqueSections);
	for (int32 SectionIndex = 0; SectionIndex < NumUniqueSections; SectionIndex++)
	{
		TArray<FQuat4f> &SectionRefToLocalQuats = PerSectionRefToLocalQuats[SectionIndex];
		FRDGBufferSRVRef &SectionRefToLocalQuatsSRV = PerSectionRefToLocalQuatsSRVs[SectionIndex];
		const FSkelMeshRenderSection& Section = LodRenderData->RenderSections[SectionIndex];
		const uint32 LocalNumBones = Section.BoneMap.Num();

		SectionRefToLocalQuats.SetNum(LocalNumBones);
		for (uint32 BoneIndex = 0; BoneIndex < LocalNumBones; BoneIndex++)
		{
			const FBoneIndexType RefToLocalIndex = Section.BoneMap[BoneIndex];
			check(RefToLocalQuats.IsValidIndex(RefToLocalIndex));
			SectionRefToLocalQuats[BoneIndex] = RefToLocalQuats[RefToLocalIndex];
		}

		if (LocalNumBones <= 0)
		{
			SectionRefToLocalQuatsSRV = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(GWhiteVertexBufferWithRDG->Buffer), PF_A32B32G32R32F);
		}
		else
		{
			FRDGBufferRef SectionRefToLocalQuatsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(FQuat4f), LocalNumBones), TEXT("SectionRefToLocalQuats"));
			SectionRefToLocalQuatsSRV = GraphBuilder.CreateSRV(SectionRefToLocalQuatsBuffer, PF_A32B32G32R32F);
			GraphBuilder.QueueBufferUpload(SectionRefToLocalQuatsBuffer, SectionRefToLocalQuats.GetData(), LocalNumBones * sizeof(FQuat4f));
		}
	}
}

void FOptimusSkeletonWithQuatsDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];
	const bool bUnifiedBoneMap = SkeletalMeshRenderData.HasUnifiedBoneMap();

	FRHIShaderResourceView* NullSRVBinding = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI.GetReference();

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		const int32 BoneBufferIndex = bUnifiedBoneMap ? 0 : InvocationIndex;

		const bool bPreviousFrame = false;
		FRHIShaderResourceView* BoneBufferSRV = FSkeletalMeshDeformerHelpers::GetBoneBufferForReading(SkeletalMeshObject, LodIndex, InvocationIndex, bPreviousFrame);

		FSkinWeightVertexBuffer const* WeightBuffer = LodRenderData->GetSkinWeightVertexBuffer();
		check(WeightBuffer != nullptr);
		FRHIShaderResourceView* SkinWeightBufferSRV = WeightBuffer->GetDataVertexBuffer()->GetSRV();
		const bool bUnlimitedBoneInfluences = WeightBuffer->GetBoneInfluenceType() == GPUSkinBoneInfluenceType::UnlimitedBoneInfluence;
		FRHIShaderResourceView* InputWeightLookupStreamSRV = bUnlimitedBoneInfluences ? WeightBuffer->GetLookupVertexBuffer()->GetSRV() : nullptr;

		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices = LodRenderData->GetNumVertices();
		Parameters.NumBoneInfluences = WeightBuffer->GetMaxBoneInfluences();
		Parameters.InputWeightStride = WeightBuffer->GetConstantInfluencesVertexStride();
		Parameters.InputWeightIndexSize = WeightBuffer->GetBoneIndexByteSize() | (WeightBuffer->GetBoneWeightByteSize() << 8);
		Parameters.BoneMatrices = BoneBufferSRV != nullptr ? BoneBufferSRV : NullSRVBinding;
		Parameters.BoneQuats = PerSectionRefToLocalQuatsSRVs.IsValidIndex(BoneBufferIndex) ? PerSectionRefToLocalQuatsSRVs[BoneBufferIndex] : nullptr;
		Parameters.InputWeightStream = SkinWeightBufferSRV != nullptr ? SkinWeightBufferSRV : NullSRVBinding;
		Parameters.InputWeightLookupStream = InputWeightLookupStreamSRV != nullptr ? InputWeightLookupStreamSRV : NullSRVBinding;
	}
}

void FOptimusSkeletonWithQuatsDataProviderProxy::CacheRefToLocalQuats(TArray<FQuat4f>& OutRefToLocalQuats) const
{
	OutRefToLocalQuats.Reset();

	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject || !SkeletalMeshObject->HaveValidDynamicData())
	{
		return;
	}

	TConstArrayView<FMatrix44f> RefToLocalMatrices = SkeletalMeshObject->GetReferenceToLocalMatrices();
	const int32 NumQuats = RefToLocalMatrices.Num();
	OutRefToLocalQuats.SetNumUninitialized(NumQuats);

	/** It is theoretically more efficient to compute q_ref ^ {-1} * q_local
	 *  Since we don't have access to q_ref in SkeletalMeshObject, we convert simply matrices to quaternions 
	 */
	for (int32 QuatIdx = 0; QuatIdx < NumQuats; ++QuatIdx)
	{
		OutRefToLocalQuats[QuatIdx] = RefToLocalMatrices[QuatIdx].GetMatrixWithoutScale().ToQuat();
	}
}

