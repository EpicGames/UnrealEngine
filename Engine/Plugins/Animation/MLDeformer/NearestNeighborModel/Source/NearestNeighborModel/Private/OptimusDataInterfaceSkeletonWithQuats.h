// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "PrimitiveComponentId.h"
#include "RenderGraphFwd.h"

#include "OptimusDataInterfaceSkeletonWithQuats.generated.h"

#define UE_API NEARESTNEIGHBORMODEL_API

class FSceneInterface;
class FSkeletalMeshObject;
class FSkeletonWithQuatsDataInterfaceParameters;
class USkinnedMeshComponent;

/** Compute Framework Data Interface for skeletal data. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusSkeletonWithQuatsDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("SkeletonWithQuats"); }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	static UE_API TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading skeletal mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusSkeletonWithQuatsDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<USkinnedMeshComponent> SkinnedMesh = nullptr;

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusSkeletonWithQuatsDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusSkeletonWithQuatsDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherPermutations(FPermutationData& InOutPermutationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FSkeletonWithQuatsDataInterfaceParameters;
	void CacheRefToLocalQuats(TArray<FQuat4f>& OutRefToLocalQuats) const;

	FSceneInterface* SceneInterface = nullptr;
	FPrimitiveComponentId ComponentId;
	uint32 BoneRevisionNumber = 0;
	TArray<TArray<FQuat4f> > PerSectionRefToLocalQuats;
	TArray<FRDGBufferSRVRef> PerSectionRefToLocalQuatsSRVs;
};

#undef UE_API
