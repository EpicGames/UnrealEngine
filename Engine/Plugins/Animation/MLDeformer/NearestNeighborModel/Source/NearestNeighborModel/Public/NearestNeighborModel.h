// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

#include "MLDeformerMorphModel.h"
#include "NearestNeighborModelHelpers.h"

#include "NearestNeighborModel.generated.h"

#define UE_API NEARESTNEIGHBORMODEL_API

class UAnimSequence;
class UGeometryCache;
class UNearestNeighborOptimizedNetwork;
struct FMLDeformerGeomCacheTrainingInputAnim;


namespace UE::NearestNeighborModel
{
#if WITH_EDITOR
	class FNearestNeighborModelDetails;
#endif
};

USTRUCT(BlueprintType, Blueprintable)
struct UE_DEPRECATED(5.4, "FClothPartData is deprecated. Use UNearestNeighborModelSection instead.") FClothPartData
{
	GENERATED_BODY()

	/** Number of PCA coefficients for this cloth part. */
	UPROPERTY()
	int32 PCACoeffNum = 128;

	/** The start index of PCA coeffcients of this cloth part */
	UPROPERTY()
	uint32 PCACoeffStart = 0;

	/** Number of vertices in this cloth part */
	UPROPERTY()
	uint32 NumVertices = 0;

	/** Number of neighbors used for this cloth part */
	UPROPERTY()
	uint32 NumNeighbors = 0;

	/** Vertex indices for this cloth part */
	UPROPERTY()
	TArray<uint32> VertexMap;
	
	/** PCA basis for this cloth part. This is a flattened array of size (PCACoeffNum, NumVertices * 3)  */
	UPROPERTY()
	TArray<float> PCABasis;

	/** Vertex mean for PCA computation. This has the size of NumVertices * 3 */
	UPROPERTY()
	TArray<float> VertexMean;

	/** PCA coefficients of the nearest neighbors. This is a flattened array of size (NumNeighbors, PCACoeffNum) */
	UPROPERTY()
	TArray<float> AssetNeighborCoeffs;

	/** The remaining offsets of the nearest neighbor shapes (after reducing PCA offsets). This is a flattened array of size (NumNeighbors, PCACoeffNum) */
	UPROPERTY()
	TArray<float> AssetNeighborOffsets;

	/** Mapping from NeighborCoeffs to AssetNeighborCoeffs */
	UPROPERTY()
	TArray<int32> AssetNeighborIndexMap;

	UPROPERTY()
	TArray<float> NeighborCoeffs;
};

#if WITH_EDITORONLY_DATA
UENUM()
enum class ENearestNeighborModelSectionWeightMapCreationMethod : uint8
{
	/** Include all vertices from text with weight 1. */
	FromText,
	/** Use skinning weights from selected bones. */
	SelectedBones,
	/** Use weights from a vertex attribute. */
	VertexAttributes,
	/** Using an external .txt file */
	ExternalTxt
};
#endif

/**
 * The section of the nearest neighbor model.
 * Each section contains a set of vertices in the original skeletal mesh.
 * The nearest neighbor search is performed on each section separately.
 * For example, if a character asset has both shirt and pants mesh, 
 * user can define sperate two separate sections for shirt and pants, 
 * and the nearest neighbor search will be performed separately on those two sections.
 */
UCLASS(MinimalAPI)
class UNearestNeighborModelSection : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FPropertyChangedDelegate, FPropertyChangedEvent&, UNearestNeighborModelSection&);
	using EOpFlag = UE::NearestNeighborModel::EOpFlag;

	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API int32 GetNumBasis() const;

	UE_API int32 GetNumVertices() const;
	UE_API int32 GetRuntimeNumNeighbors() const;

	UE_API bool IsReadyForTraining() const;
	UE_API bool IsReadyForInference() const;

	UE_API const TArray<float>& GetNeighborCoeffs() const;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	UE_API void SetNumBasis(const int32 InNumBasis);

	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API int32 GetAssetNumNeighbors() const;
	
	UE_API const UAnimSequence* GetNeighborPoses() const;
	UE_API const UGeometryCache* GetNeighborMeshes() const;
	// Avoid mutable getters. Changing NeighborPoses or NeighborMeshes may cause unexpected errors.  
	UE_API UAnimSequence* GetMutableNeighborPoses() const;
	UE_API UGeometryCache* GetMutableNeighborMeshes() const;
	
	UE_API void SetVertexMapString(const FString& InString);

	UE_API int32 GetMeshIndex() const;
	UE_API void SetMeshIndex(int32 Index);

	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API const TArray<int32>& GetVertexMap() const;
	
	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API const TArray<float>& GetVertexWeights() const;
	
	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API const TArray<float>& GetBasis() const;
	
	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API const TArray<float>& GetVertexMean() const;
	
	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API const TArray<float>& GetAssetNeighborCoeffs() const;

	UFUNCTION(BlueprintPure, Category = "Section")
	UE_API bool DoesUsePCA() const;

	UE_API const TArray<float>& GetAssetNeighborOffsets() const;
	UE_API const TArray<int32>& GetExcludedFrames() const;
	UE_API const TArray<int32>& GetAssetNeighborIndexMap() const;

	UE_API FMLDeformerGeomCacheTrainingInputAnim* GetInputAnim() const;

	UE_API EOpFlag UpdateVertexWeights();

	UE_API EOpFlag UpdateForTraining();
	UE_API EOpFlag UpdateForInference();
	UE_API void InvalidateTraining();
	UE_API void InvalidateInference();

	UE_API bool UpdateRuntimeNeighbors();
	UE_API bool GetRuntimeNeighborOffsets(TArray<float>& OutNeighborOffsets) const;

	UE_API void SetModel(UNearestNeighborModel* InModel);
	UE_API const UNearestNeighborModel* GetModel() const;

	UE_API ENearestNeighborModelSectionWeightMapCreationMethod GetWeightMapCreationMethod() const;
	UE_API FString GetBoneNamesString() const;
	UE_API const TArray<FName>& GetBoneNames() const;
	UE_API void SetBoneNames(const TArray<FName>& InBoneNames);
	UE_API FString GetExternalTxtFile() const;
	UE_API void SetExternalTxtFile(const FString& InFile);


	// Do not call this function directly. Call UNearestNeighborModel::NormalizeVertexWeights() instead.
	UE_API EOpFlag NormalizeVertexWeights();

	UE_API void ClearReferences();
	UE_API void FinalizeMorphTargets();


	static FName GetNumBasisPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, NumPCACoeffs); }
	static FName GetVertexMapStringPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, VertexMapString); }
	static FName GetNeighborPosesPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, NeighborPoses); }
	static FName GetNeighborMeshesPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, NeighborMeshes); }
	static FName GetExcludedFramesPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, ExcludedFrames); }
	static FName GetWeightMapCreationMethodPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, WeightMapCreationMethod); }
	static FName GetAttributeNamePropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, AttributeName); }
	static FName GetExternalTxtFilePropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModelSection, ExternalTxtFile); }
#endif
protected:
#if WITH_EDITORONLY_DATA
	/** Poses of the nearest neighbor ROM. */
	UPROPERTY(EditAnywhere, Category = "Section", meta = (DisplayName = "Neighbor Sequence"))
	TObjectPtr<UAnimSequence> NeighborPoses;

	/** Geometry cache of the nearest neighbor ROM. */
	UPROPERTY(EditAnywhere, Category = "Section", meta = (DisplayName = "Neighbor Geom Cache"))
	TObjectPtr<UGeometryCache> NeighborMeshes;

	/** Method to create weight map for this section. */
	UPROPERTY(EditAnywhere, Category = "Section")
	ENearestNeighborModelSectionWeightMapCreationMethod WeightMapCreationMethod = ENearestNeighborModelSectionWeightMapCreationMethod::FromText;

	/** Bone names used to create weight map. */
	UPROPERTY()
	TArray<FName> BoneNames;
	
	/** A float vertex attribute that is used to compute weight maps */
	UPROPERTY(EditAnywhere, Category = "Section", meta = (GetOptions = "GetVertexAttributeNames", NoResetToDefault))
	FName AttributeName;

	/** A string containing vertex indices for this section, e.g. "2, 3, 5-8, 9, 11-20" */
	UPROPERTY(EditAnywhere, Category = "Section", meta = (DisplayName = "Vertex Indices"))
	FString VertexMapString;

	/** The path to the txt file containing vertex weights. The number of lines equals to the number of vertices in skeletal mesh with each line being a float for vertex weight. */
	UPROPERTY(VisibleAnywhere, Category = "Section")
	FString ExternalTxtFile;

	/** Frames to be excluded from the nearest neighbor ROM */
	UPROPERTY(EditAnywhere, Category = "Section")
	TArray<int32> ExcludedFrames;

	/** The mesh index in SkeletalMeshRenderData */
	UPROPERTY()
	int32 MeshIndex = 0;

	/** The parent nearest neighbor model */
	UPROPERTY()
	TObjectPtr<UNearestNeighborModel> Model = nullptr;

	/** Vertex indices for this section. VertexMap.Num() == NumVertices */
	UPROPERTY()
	TArray<int32> VertexMap;

	/** The vertex weights for this section. VertexWeights.Num() == NumVertices. */
	UPROPERTY()
	TArray<float> VertexWeights;

	/** Flattened array of basis. The shape of basis is (CoeffNum, NumVertices * 3)  */
	UPROPERTY()
	TArray<float> Basis;
	
	/** The vertex mean on the shape. This array has a size of NumVertices * 3 */
	UPROPERTY()
	TArray<float> VertexMean;

	/** Flattened array of neighbor coefficients before excluding frames. The shape of this array is (NumNeighbors, NumCoeffs) */
	UPROPERTY()
	TArray<float> AssetNeighborCoeffs;
	
	/** Flattened array of neighbor offsets before excluding frames. The shape of this array is (NumNeighbors, NumVertices * 3) */
	UPROPERTY()
	TArray<float> AssetNeighborOffsets;

	/** The index into the original nearest neighbor ROM asset after frames are excluded. */
	UPROPERTY()
	TArray<int32> AssetNeighborIndexMap;
#endif

	/** Number of PCA coefficients for this section. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (ClampMin = "1", DefaultValue = "64", EditCondition = "DoesUsePCA()"), Category = "Section", DisplayName = "Num Basis") // TODO: rename to NumBasis
	int32 NumPCACoeffs = 64;

	/** Number of vertices in this section */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Section Private")
	int32 NumVertices = 0;
	
	/** Number of neighbors in this section after excluding frames */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Section Private")
	int32 RuntimeNumNeighbors = 0;
	
	/** Flattened array of neighbor offsets after excluding frames. */
	UPROPERTY()
	TArray<float> RuntimeNeighborCoeffs;

private:
	/** Whether the section is ready for training */
	UPROPERTY()
	bool bIsReadyForTraining = false;
	
	/** Whether the section is ready for inference */
	UPROPERTY()
	bool bIsReadyForInference = false;

#if WITH_EDITOR
	UE_API bool IsBasisValid() const;
	UE_API bool IsBasisEmpty() const;
	UE_API bool IsNearestNeighborValid() const;
	UE_API bool IsNearestNeighborEmpty() const;

	UFUNCTION(BlueprintCallable, Category = "Section")
	UE_API void SetBasisData(const TArray<float>& InVertexMean, const TArray<float>& InBasis);
	
	UFUNCTION(BlueprintCallable, Category = "Section")
	UE_API void SetNeighborData(const TArray<float>& InNeighborCoeffs, const TArray<float>& InNeighborOffsets);

	UE_API void Reset();
	UE_API void ResetBasisData();
	UE_API void ResetNearestNeighborData();
	UE_API EOpFlag UpdateVertexWeightsFromText();
	UE_API EOpFlag UpdateVertexWeightsSelectedBones();
	UE_API EOpFlag UpdateVertexWeightsVertexAttributes();
	UE_API EOpFlag UpdateVertexWeightsExternalTxt();

	UFUNCTION()
	UE_API TArray<FName> GetVertexAttributeNames() const;
	
	/** A temporary InputAnim used by GetInputAnim() function. */
	mutable TUniquePtr<FMLDeformerGeomCacheTrainingInputAnim> InputAnim;
#endif
};

/**
 * The nearest neighbor model.
 * This model contains the linear basis of the vertex deltas and a small set of meshes for nearest neighbor search. 
 * Given a new pose, the pre-trained neural network first predicts the coefficients of the vertex deltas. 
 * Then this model uses the predicted coeffcients to find a nearest neighbor in the small dataset.
 * The total vertex delta is computed by
 * 		vertex_delta = mean_delta + basis * coeff + nearest_neighbor_delta
 * To prevent popping, a time filtering is applied on predicted vertex deltas. The vertex delta at time t is computed by  
 * 		vertex_delta(t) = decay_factor * vertex_delta(t-1) + (1 - decay_factor) * vertex_delta 
 * The mesh can be separated into several sections (e.g. shirt, pants...). The nearest neighbor search is carried out separately for each section. 
 * The basis and the nearest neighbor data are compressed into morph targets.
 */
UCLASS(MinimalAPI)
class UNearestNeighborModel
	: public UMLDeformerMorphModel 
{
	GENERATED_BODY()

public:
	using FSection = UNearestNeighborModelSection;
	using UNetwork = UNearestNeighborOptimizedNetwork;

	UE_API UNearestNeighborModel(const FObjectInitializer& ObjectInitializer);

	// UObject overrides.
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	UE_API virtual void FinalizeMorphTargets() override;
#endif
	// ~END UObject overrides.

	// UMLDeformerModel overrides.
	UE_API virtual void PostLoad() override;
	UE_API virtual void Serialize(FArchive& Archive) override;
	UE_API virtual UMLDeformerModelInstance* CreateModelInstance(UMLDeformerComponent* Component) override;
	UE_API virtual UMLDeformerInputInfo* CreateInputInfo() override;
	UE_API virtual FString GetDisplayName() const override;
	virtual int32 GetNumFloatsPerBone() const override { return NearestNeighborNumFloatsPerBone; }
	virtual int32 GetNumFloatsPerCurve() const override { return NearestNeighborNumFloatsPerCurve; }
	UE_API virtual bool IsTrained() const override;
	UE_API virtual FString GetDefaultDeformerGraphAssetPath() const;
	// ~END UMLDeformerModel overrides.

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	UE_API int32 GetNumSections() const;

	// GetSectionPtr is reserved for python. Use GetSection for C++.
	UFUNCTION(BlueprintPure, Category = "Python")
	UE_API const UNearestNeighborModelSection* GetSectionPtr(int32 Index) const;
	UE_API const UNearestNeighborModelSection& GetSection(int32 Index) const;
	UE_API UNearestNeighborModelSection& GetSection(int32 Index);

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	UE_API const TArray<int32>& GetPCACoeffStarts() const;

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	UE_API int32 GetTotalNumBasis() const;
	UE_API int32 GetNumBasisPerSection() const;

	UE_API int32 GetTotalNumNeighbors() const;
	UE_API float GetDecayFactor() const;
	UE_API float GetNearestNeighborOffsetWeight() const;

	UE_API TWeakObjectPtr<const UNetwork> GetOptimizedNetwork() const;
	UE_API int32 GetNumNetworkOutputs() const;
	UE_API void ClipInputs(TArrayView<float> Inputs) const;

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	UE_API bool IsReadyForTraining() const;

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	UE_API bool IsReadyForInference() const;

	UE_API bool DoesUseRBF() const;
	UE_API float GetRBFSigma() const;

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	bool DoesUsePCA() const { return bUsePCA; }

#if WITH_EDITORONLY_DATA
	UE_API TArray<FInt32Range> GetMeshVertRanges(const USkeletalMesh& SkelMesh);
#endif

#if WITH_EDITOR
	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	int32 GetInputDim() const { return InputDim; }
	
	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	TArray<int32> GetHiddenLayerDims() const { return HiddenLayerDims; }
	
	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	int32 GetOutputDim() const { return OutputDim; }
	
	UE_DEPRECATED(5.5, "GetNumEpochs() is deprecated.")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "GetNumEpochs has been deprecated. Convert from GetNumIterations instead."))
	int32 GetNumEpochs() const { return 0; }

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	int32 GetNumIterations() const { return NumIterations; }
	
	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	int32 GetBatchSize() const { return BatchSize; }
	
	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	float GetLearningRate() const { return LearningRate; }

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	int32 GetEarlyStopEpochs() const { return EarlyStopEpochs; }

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	float GetRegularizationFactor() const { return RegularizationFactor; }

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	float GetSmoothLossBeta() const { return SmoothLossBeta; }

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	UE_API FString GetModelDir() const;
	
	bool DoesUseFileCache() const { return bUseFileCache; }
	bool DoesUseDualQuaternionDeltas() const { return bUseDualQuaternionDeltas; }

	UE_API FDateTime GetNetworkLastWriteTime() const;
	UE_API FString GetNetworkLastWriteArchitectureString() const;
	UE_API FDateTime GetMorphTargetsLastWriteTime() const;

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	UE_API TArray<FString> GetCachedDeltasPaths() const;

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	UE_API TArray<FString> GetCachedPCAPaths() const;

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	UE_API TArray<FString> GetCachedNetworkPaths() const;
	
	UE_API TOptional<FDateTime> GetCachedDeltasTimestamp() const;
	UE_API TOptional<FDateTime> GetCachedPCATimestamp() const;
	UE_API TOptional<FDateTime> GetCachedNetworkTimestamp() const;

	using EOpFlag = UE::NearestNeighborModel::EOpFlag;
	UE_API EOpFlag UpdateForTraining();
	UE_API void InvalidateTraining();
	UE_API void InvalidateTrainingModelOnly();
	UE_API EOpFlag UpdateForInference();
	UE_API void InvalidateInference();
	UE_API void InvalidateInferenceModelOnly();
	UE_API bool LoadOptimizedNetworkFromFile(const FString& Filename);
	UE_API void ClearOptimizedNetwork();
	UE_API void RemoveAllSections();

	UE_API FMLDeformerGeomCacheTrainingInputAnim* GetNearestNeighborAnim(int32 SectionIndex);
	UE_API const FMLDeformerGeomCacheTrainingInputAnim* GetNearestNeighborAnim(int32 SectionIndex) const;

	void AddSection(TObjectPtr<UNearestNeighborModelSection> Section) { Sections.Add(Section); }

	UE_API void UpdateFileCache();
	UE_API const FString& GetFileCacheDirectory() const;
	UE_API void SetFileCacheDirectory(const FString& InFileCacheDirectory);

	/** Make sure to call this function after changing the morph targets.
	* Do not call this function without calling InitEngineMorphTargets first. */
	UE_API void UpdateMorphTargetsLastWriteTime();

	UE_API void UpdateNetworkInputDim();
	UE_API void UpdateNetworkOutputDim();

	UE_API void ClearReferences();

	UE_API bool IsBeforeCustomVersionWasAdded() const;
	UE_API bool IsBeforeTrainedBasisAdded() const;
	UE_API bool IsBeforeDeprecateNumEpochs() const;
	UE_API const TArray<float>& GetVertexWeightSum() const;

	static FName GetInputDimPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, InputDim); }
	static FName GetHiddenLayerDimsPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, HiddenLayerDims); }
	static FName GetOutputDimPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, OutputDim); }
	static FName GetNumIterationsPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, NumIterations); }
	static FName GetBatchSizePropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, BatchSize); }
	static FName GetLearningRatePropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, LearningRate); }
	static FName GetEarlyStopEpochsPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, EarlyStopEpochs); }
	static FName GetSectionsPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, Sections); }
	static FName GetNearestNeighborOffsetWeightPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, NearestNeighborOffsetWeight); }
	static FName GetUseFileCachePropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, bUseFileCache); }
	static FName GetFileCacheDirectoryPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, FileCacheDirectory); }
	static FName GetUsePCAPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, bUsePCA); }
	static FName GetNumBasisPerSectionPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, NumBasisPerSection); }
	static FName GetUseDualQuaternionDeltasPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, bUseDualQuaternionDeltas); }
	static FName GetDecayFactorPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, DecayFactor); }
	static FName GetUseRBFPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, bUseRBF); }
	static FName GetRBFSigmaPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModel, RBFSigma); }
#endif
	
protected:
	/** Network input dimensions. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Training Settings")
	int32 InputDim = 0;

	/** Network output dimensions. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Training Settings")
	int32 OutputDim = 0;

#if WITH_EDITORONLY_DATA
	/** Dimension of hidden layers in the network. This cannot be empty. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings")
	TArray<int32> HiddenLayerDims;

	/** Max number of cycles iterated through the training set. */
	UE_DEPRECATED(5.5, "NumEpochs is deprecated. Convert to NumIterations instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "NumEpochs is deprecated. Convert to NumIterations instead.", ClampMin = "1"))
	int32 NumEpochs_DEPRECATED = 2500;

	/** The number of iterations to train the model for. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings", meta = (ClampMin = "1", ClampMax = "1000000"))
	int32 NumIterations = 5000;

	/** Number of data samples processed together as a group in a single pass. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings", AdvancedDisplay, meta = (ClampMin = "1"))
	int32 BatchSize = 256;

	/** The regularization factor. Higher values can help generate more sparse morph targets, but can also lead to visual artifacts.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings", AdvancedDisplay, meta = (ClampMin = "0", EditCondition = "!bUsePCA"))
	float RegularizationFactor = 1.0f;
	
	/** The beta parameter in the smooth L1 loss function, which describes below which absolute error to use a squared term. If the error is above or equal to this beta value, it will use the L1 loss. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Training Settings", meta = (ClampMin = "0.0", EditCondition = "!bUsePCA"))
	float SmoothLossBeta = 1.0f;

	/** The size of the step when optimizing the network. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings", AdvancedDisplay, meta = (ClampMin = "0.000001", ClampMax = "1.0"))
	float LearningRate = 0.001f;

	/** The number of epochs to stop training if there is no improvement in accuracy. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings", AdvancedDisplay, meta = (ClampMin = "1", EditCondition = "bUsePCA"))
	int32 EarlyStopEpochs = 100;

	/** Whether to cache intermediate results on disk. CAUTION: failing to manually clear caches could cause unexpected results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Cache", meta = (DisplayName = "Use File Cache (Advanced)"))
	bool bUseFileCache = false;

	/** Directory to save the intermediate results. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "File Cache", meta = (EditCondition = "bUseFileCache", EditConditionHides))
	FString FileCacheDirectory = FPaths::ProjectIntermediateDir() + "NearestNeighborModel";

	TOptional<FDateTime> CachedDeltasTimestamp;
	TOptional<FDateTime> CachedPCATimestamp;
	TOptional<FDateTime> CachedNetworkTimestamp;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.4, "ClothPartData is deprecated. Use Sections instead.")
	UPROPERTY()
	TArray<FClothPartData> ClothPartData_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	static constexpr int32 NearestNeighborNumFloatsPerBone = 3;
	static constexpr int32 NearestNeighborNumFloatsPerCurve = 0;

	/** Whether to use input multipliers. This can be used to debug bad network input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debugging")
	bool bUseInputMultipliers = false;

	/** Values to be multiplied to the input. This can be used to debug bad network input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debugging")
	TArray<FVector3f> InputMultipliers;

	/** The min input values observed throughout the entire training set. This is used to clamp the input value at inference time.  The values are set in python. */
	UPROPERTY(BlueprintReadWrite, Category = "Network IO")
	TArray<float> InputsMin;

	/** The max input values observed throughout the entire training set. This is used to clamp the input value at inference time.  The values are set in python. */
	UPROPERTY(BlueprintReadWrite, Category = "Network IO")
	TArray<float> InputsMax;
	
	/** Whether to use pre-computed PCA basis. If false, basis will be learned at training time. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Nearest Neighbor Settings")
	bool bUsePCA = false;

	/** The number of basis used in each section. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Nearest Neighbor Settings", meta = (EditCondition = "!bUsePCA", EditConditionHides))
	int32 NumBasisPerSection = 128;

	/** Whether to use dual quaternion deltas. If false, LBS deltas will be used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Nearest Neighbor Settings")
	bool bUseDualQuaternionDeltas = true;

	/** The ratio of previous frame deltas added into the current frame deltas. A bigger value will make wrinkles "stick" longer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Nearest Neighbor Settings", META = (ClampMin = "0", ClampMax = "1"))
	float DecayFactor = 0.85f;

	/** The weight multiplied to nearest neighbor deltas. A value of 0 means completely removing nearest neighbor deltas. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Nearest Neighbor Settings", META = (ClampMin = "0", ClampMax = "1"))
	float NearestNeighborOffsetWeight = 1.0f;

	/** Whether to use radial basis function to blend multiple nearest neighbors to produce smoother result. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Nearest Neighbor Settings")
	bool bUseRBF = false;

	/** Range to blend nearest neighbors. A bigger range will blend more neighbors, produce smoother result and may be slower in game. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Nearest Neighbor Settings", META = (ClampMin = "0"))
	float RBFSigma = 1.0f;

public:
#if WITH_EDITOR
	UE_API UNearestNeighborModelSection* OnSectionAdded(int32 NewIndex);
#endif

private:
	UE_API TWeakObjectPtr<UNetwork> GetOptimizedNetwork();

#if WITH_EDITOR

	UE_API void SetOptimizedNetwork(UNearestNeighborOptimizedNetwork* InOptimizedNetwork);

	UE_API void UpdatePCACoeffStarts();

	UE_API EOpFlag CheckHiddenLayerDims();
	UE_API void UpdateInputMultipliers();

	UE_API void UpdateSectionNumBasis();
	UE_API void UpdateCachedDeltasTimestamp();
	UE_API void UpdateCachedPCATimestamp();
	UE_API void UpdateCachedNetworkTimestamp();

	UE_API void NormalizeVertexWeights();
	UE_API void UpdateVersion();
	// FNearestNeighborModelDetails needs to call private function GetSection(int32).
	friend class UE::NearestNeighborModel::FNearestNeighborModelDetails;
#endif

	UPROPERTY(EditAnywhere, Category = "Mesh Sections")
	TArray<TObjectPtr<UNearestNeighborModelSection>> Sections;
	
	UPROPERTY()
	TArray<int32> PCACoeffStarts;
	
	UPROPERTY()
	bool bIsReadyForTraining = false;

	UPROPERTY()
	bool bIsReadyForInference = false;

	UPROPERTY()
	TObjectPtr<UNearestNeighborOptimizedNetwork> OptimizedNetwork;

	int32 Version = INDEX_NONE;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	int64 NetworkLastWriteTime = 0;

	UPROPERTY()
	int64 MorphTargetsLastWriteTime = 0;

	UPROPERTY()
	FString NetworkLastWriteArchitectureString;

	TArray<float> VertexWeightSum;
#endif
};

#undef UE_API
