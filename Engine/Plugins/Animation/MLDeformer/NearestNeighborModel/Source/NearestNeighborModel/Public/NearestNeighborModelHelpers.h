// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "UObject/UObjectGlobals.h"

#define UE_API NEARESTNEIGHBORMODEL_API

class UAnimSequence;
class UGeometryCache;

NEARESTNEIGHBORMODEL_API DECLARE_LOG_CATEGORY_EXTERN(LogNearestNeighborModel, Log, All);

namespace UE::NearestNeighborModel
{
	// Use a helper class to avoid adding NEARESTNEIGHBORMODEL_API to all functions.  
	class FHelpers
	{
	public:
#if WITH_EDITOR
		static UE_API int32 GetNumFrames(const UAnimSequence* AnimSequence);
		static UE_API int32 GetNumFrames(const UGeometryCache* GeometryCache);
		
		template<class T>
		static T* NewDerivedObject()
		{
			TArray<UClass*> Classes;
			GetDerivedClasses(T::StaticClass(), Classes);
			if (Classes.IsEmpty())
			{
				return nullptr;
			}
			return NewObject<T>(GetTransientPackage(), Classes.Last());
		}
#endif
	
		template<typename T>
		static TArray<T> Range(T End)
		{
			TArray<T> Result;
			Result.SetNum(End);
			for (uint32 i = 0; (T)i < End; i++)
			{
				Result[i] = i;
			}
			return Result;
		}

		template<typename T>
		static TArray<T> Range(const T Start, const T End)
		{
			TArray<T> Arr; Arr.SetNum(End - Start);
			for (T i = Start; (T)i < End; i++)
			{
				Arr[i - Start] = i;
			}
			return Arr;
		}
	};

	enum class EOpFlag : uint8
	{
		Success = 0,
		Error = 1,
		Warning = 2
	};
	
	namespace OpFlag
	{
		NEARESTNEIGHBORMODEL_API bool HasError(EOpFlag Flag);
		NEARESTNEIGHBORMODEL_API EOpFlag AddError(EOpFlag Flag);
		NEARESTNEIGHBORMODEL_API bool HasWarning(EOpFlag Flag);
		NEARESTNEIGHBORMODEL_API EOpFlag AddWarning(EOpFlag Flag); 
	};
	NEARESTNEIGHBORMODEL_API EOpFlag operator|(EOpFlag A, EOpFlag B);
	NEARESTNEIGHBORMODEL_API EOpFlag& operator|=(EOpFlag& A, EOpFlag B);
	NEARESTNEIGHBORMODEL_API EOpFlag operator&(EOpFlag A, EOpFlag B);
	NEARESTNEIGHBORMODEL_API EOpFlag& operator&=(EOpFlag& A, EOpFlag B);
	NEARESTNEIGHBORMODEL_API EOpFlag ToOpFlag(int32 Value);
};

#undef UE_API
