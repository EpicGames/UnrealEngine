// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MLDeformerMorphModelInputInfo.h"
#include "NearestNeighborModelInputInfo.generated.h"

#define UE_API NEARESTNEIGHBORMODEL_API

namespace UE::NearestNeighborModel
{
    class FNearestNeighborEditorModel;
};

UCLASS(MinimalAPI)
class UNearestNeighborModelInputInfo
    : public UMLDeformerMorphModelInputInfo
{
    GENERATED_BODY()

public:
    // UMLDeformerInputInfo overrides
    UE_API virtual int32 CalcNumNeuralNetInputs(int32 NumFloatsPerBone, int32 NumFloatsPerCurve) const override;
    UE_API virtual void ExtractBoneRotations(USkeletalMeshComponent* SkelMeshComponent, TArray<float>& OutRotations) const override;
    // ~END UMLDeformerInputInfo overrides

    UE_API void ComputeNetworkInput(TConstArrayView<FQuat> BoneRotations, TArrayView<float> OutputView) const;

private:
    friend class UE::NearestNeighborModel::FNearestNeighborEditorModel;

    UE_API void InitRefBoneRotations(const USkeletalMesh* SkelMesh);

private:
    UPROPERTY()
    TArray<FQuat> RefBoneRotations;
};

#undef UE_API
