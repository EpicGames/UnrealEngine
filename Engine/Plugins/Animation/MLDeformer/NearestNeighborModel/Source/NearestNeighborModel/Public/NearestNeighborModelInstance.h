// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MLDeformerMorphModelInstance.h"
#include "Misc/Optional.h"
#include "NearestNeighborModelInstance.generated.h"

#define UE_API NEARESTNEIGHBORMODEL_API

class UNearestNeighborModel;
class UNearestNeighborModelInputInfo;
class UNearestNeighborModelInstance;
class UNearestNeighborOptimizedNetwork;
class UNearestNeighborOptimizedNetworkInstance;
namespace UE::NearestNeighborModel
{
    class FNearestNeighborEditorModel;
    class FNearestNeighborEditorModelActor;
};

UCLASS(MinimalAPI, Blueprintable)
class UNearestNeighborModelInstance
    : public UMLDeformerMorphModelInstance
{
    GENERATED_BODY()

public:
	// UMLDeformerModelInstance overrides
    UE_API virtual void Init(USkeletalMeshComponent* SkelMeshComponent) override;
    UE_API virtual void Execute(float ModelWeight) override;
    UE_API virtual bool SetupInputs() override;
    UE_API virtual FString CheckCompatibility(USkeletalMeshComponent* InSkelMeshComponent, bool LogIssues=false) override;
    UE_API virtual void Tick(float DeltaTime, float ModelWeight) override;
    UE_API virtual int64 SetBoneTransforms(float* OutputBuffer, int64 OutputBufferSize, int64 StartIndex) override;
    // ~END UMLDeformerModelInstance overrides

    UFUNCTION(BlueprintCallable, Category = "NearestNeighborModel")
    UE_API void Reset();

    friend class UNearestNeighborModelInputInfo;
    friend class UNearestNeighborModelInstance;
    friend class UNearestNeighborOptimizedNetworkInstance;
    friend class UE::NearestNeighborModel::FNearestNeighborEditorModel;
    friend class UE::NearestNeighborModel::FNearestNeighborEditorModelActor;

#if WITH_EDITOR
    UE_API TArray<uint32> GetNearestNeighborIds() const;
#endif

private:
    using UNetworkPtr = TWeakObjectPtr<UNearestNeighborOptimizedNetwork>;
    using UConstNetworkPtr = const TWeakObjectPtr<const UNearestNeighborOptimizedNetwork>;

    UE_API void InitInstanceData(int32 NumMorphWeights = INDEX_NONE);

    UE_API UNearestNeighborModel* GetCastModel() const;
    UE_API void InitOptimizedNetworkInstance();
    UE_API TOptional<TArrayView<float>> GetInputView() const;
    UE_API TOptional<TArrayView<float>> GetOutputView() const;

    UE_API void RunNearestNeighborModel(float DeltaTime, float ModelWeight);

    /** This is the slow version of network inference. It is used by python. */
    UFUNCTION(BlueprintCallable, Category = "Python")
    UE_API TArray<float> Eval(const TArray<float>& InputData);

#if WITH_EDITOR
    TArray<uint32> NearestNeighborIds;
#endif

    TArray<float> PreviousWeights;
    TArray<float> DistanceBuffer;
    bool bNeedsReset = true;

    UPROPERTY()
    TObjectPtr<UNearestNeighborOptimizedNetworkInstance> OptimizedNetworkInstance;
};

#undef UE_API
