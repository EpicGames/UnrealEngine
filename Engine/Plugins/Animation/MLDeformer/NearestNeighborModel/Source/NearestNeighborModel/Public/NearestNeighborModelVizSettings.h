// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MLDeformerMorphModelVizSettings.h"
#include "NearestNeighborModelVizSettings.generated.h"

class UGeometryCache;
namespace UE::NearestNeighborModel
{
	class FNearestNeighborModelVizSettingsDetails;
};

/**
 * The vizualization settings specific to the the vertex delta model.
 */
UCLASS(MinimalAPI)
class UNearestNeighborModelVizSettings 
	: public UMLDeformerMorphModelVizSettings
{
	GENERATED_BODY()
#if	WITH_EDITORONLY_DATA
public:
	static FName GetNearestNeighborIdsPropertyName() { return GET_MEMBER_NAME_CHECKED(UNearestNeighborModelVizSettings, NearestNeighborIds); }
	
	/** Whether to show verts */
	UPROPERTY(EditAnywhere, Category = "Training Meshes")
	bool bDrawVerts = false;

	/** Show vertices in this section */
	UPROPERTY(EditAnywhere, Category = "Training Meshes", Meta = (DisplayName = "Show Verts in", EditorCondition = "bDrawVerts"))
	int32 VertVizSectionIndex = INDEX_NONE;

	UPROPERTY()
	float NearestNeighborActorsOffset_DEPRECATED = 2.0f;

	/** The section used to display the nearest neighbor. */	
	UPROPERTY(EditAnywhere, Category = "Live Settings", Meta = (DisplayName = "Actor Section Index"))
	int32 NearestNeighborActorSectionIndex = 0;
	
	UPROPERTY(VisibleAnywhere, Category = "Live Settings")
	TArray<int32> NearestNeighborIds;

	UPROPERTY(EditAnywhere, Category = "Live Settings", meta = (DisplayName = "Part Id"))
	int32 NeighborStatsPartId = 0;

	void SetNearestNeighborIds(const TArray<int32>& InNearestNeighborIds) { NearestNeighborIds = InNearestNeighborIds; }
#endif
};
