// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "NearestNeighborOptimizedNetworkLoader.generated.h"

class UNearestNeighborOptimizedNetwork;

/** Helper class to load the optimized network from disk.
 *  LoadOptimizedNetwork is implemented in python.
 */
UCLASS()
class UE_DEPRECATED(5.4, "UNearestNeighborOptimizedNetworkLoader is deprecated. Use NNE instead.")  NEARESTNEIGHBORMODEL_API UNearestNeighborOptimizedNetworkLoader
	: public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintImplementableEvent, Category = "Nearest Neighbor Model")
	bool LoadOptimizedNetwork(const FString& OnnxPath);

	void SetOptimizedNetwork(UNearestNeighborOptimizedNetwork* InNetwork);

	UFUNCTION(BlueprintPure, Category = "Nearest Neighbor Model")
	UNearestNeighborOptimizedNetwork* GetOptimizedNetwork() const;

protected:
	TObjectPtr<UNearestNeighborOptimizedNetwork> Network = nullptr;
};