// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class NearestNeighborModelEditor : ModuleRules
	{
		public NearestNeighborModelEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"EditorFramework",
					"UnrealEd",
					"CoreUObject",
					"Engine",
					"InputCore",
					"Slate",
					"SlateCore",
					"EditorWidgets",
					"EditorStyle",
					"DesktopPlatform",
					"GeometryCache",
					"MLDeformerFramework",
					"MLDeformerFrameworkEditor",
					"NearestNeighborModel",
					"Projects",
					"PropertyEditor",
					"ToolWidgets",
					"ComputeFramework",
					"RenderCore",
					"RHI",
					"SkeletalMeshDescription",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
				}
			);
		}
	}
}
