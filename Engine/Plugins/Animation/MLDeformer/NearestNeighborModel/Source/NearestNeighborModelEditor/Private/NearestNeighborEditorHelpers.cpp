// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborEditorHelpers.h"

#include "Animation/AnimSequence.h"
#include "Animation/AttributesRuntime.h"
#include "BonePose.h"
#include "GeometryCache.h"
#include "GeometryCacheConstantTopologyWriter.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrack.h"
#include "NearestNeighborModelHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NearestNeighborEditorHelpers)

#define LOCTEXT_NAMESPACE "NearestNeighborEditorHelpers"

namespace UE::NearestNeighborModel::Private
{
	class FAnimEvaluator
	{
	public:
		FAnimEvaluator(USkeleton& Skeleton)
		{
			const FReferenceSkeleton& ReferenceSkeleton = Skeleton.GetReferenceSkeleton();
			const int32 NumBones = ReferenceSkeleton.GetNum();
			TArray<uint16> BoneIndices = ::UE::NearestNeighborModel::FHelpers::Range<uint16>(NumBones);
			BoneContainer.SetUseRAWData(true);
			BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(), Skeleton);
			OutPose.SetBoneContainer(&BoneContainer);
			OutCurve.InitFrom(BoneContainer);
		}

		TArray<FTransform> GetBoneTransforms(const UAnimSequence* Anim, int32 Frame)
		{
			TArray<FTransform> Empty;
			if (!Anim)
			{
				return Empty;
			}
			const IAnimationDataModel* DataModel = Anim->GetDataModel();
			if (!DataModel)
			{
				return Empty;
			}
			if (Frame < 0 || Frame >= Anim->GetNumberOfSampledKeys())
			{
				return Empty;
			}

			const double Time = FMath::Clamp(Anim->GetSamplingFrameRate().AsSeconds(Frame), 0., (double)Anim->GetPlayLength());
			FAnimExtractContext ExtractionContext(Time);
			FAnimationPoseData AnimationPoseData(OutPose, OutCurve, TempAttributes);
			Anim->GetAnimationPose(AnimationPoseData, ExtractionContext);

			const int32 NumBones = BoneContainer.GetNumBones();
			TArray<FTransform> Transforms;
			Transforms.SetNum(NumBones);
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				const FCompactPoseBoneIndex CompactIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(BoneIndex));
				Transforms[BoneIndex] = OutPose[CompactIndex];
			}
			return Transforms;
		}

	private:
		FBoneContainer BoneContainer;
		FCompactPose OutPose;
		FBlendedCurve OutCurve;
		UE::Anim::FStackAttributeContainer TempAttributes;
	};

	int32 GetNumVertices(UGeometryCacheTrack& Track)
	{
		FGeometryCacheMeshData MeshData;
		if (!Track.GetMeshDataAtSampleIndex(0, MeshData))
		{
			return 0;
		}
		return MeshData.Positions.Num(); 
	}
};

void UNearestNeighborAnimStream::Init(USkeleton* InSkeleton)
{
	if (!InSkeleton)
	{
		UE_LOGF(LogNearestNeighborModel, Error, "Skeleton is None when initializing NearestNeighborAnimStream");
		return;
	}
	Skeleton = InSkeleton;
	const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
	const int32 NumBones = ReferenceSkeleton.GetNum();

	PosKeys.SetNum(NumBones);
	RotKeys.SetNum(NumBones);
	ScaleKeys.SetNum(NumBones);
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		PosKeys[BoneIndex].Reset();
		RotKeys[BoneIndex].Reset();
		ScaleKeys[BoneIndex].Reset();
	}
}

bool UNearestNeighborAnimStream::IsValid() const
{
	return Skeleton != nullptr;
}

bool UNearestNeighborAnimStream::AppendFrames(const UAnimSequence* Anim, TArray<int32> Frames)
{
	if (!IsValid() || !Anim)
	{
		return false;
	}

	FMemMark Mark(FMemStack::Get());
	using UE::NearestNeighborModel::Private::FAnimEvaluator;
	FAnimEvaluator AnimEval(*Skeleton);

	const int32 NumBones = PosKeys.Num();
	if (NumBones < 1)
	{
		return false;
	}
	const int32 CurrentFrames = PosKeys[0].Num();
	const int32 NewFrames = Frames.Num();
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		check(PosKeys[Index].Num() == CurrentFrames);
		PosKeys[Index].SetNum(CurrentFrames + NewFrames);
		check(RotKeys[Index].Num() == CurrentFrames);
		RotKeys[Index].SetNum(CurrentFrames + NewFrames);
		check(ScaleKeys[Index].Num() == CurrentFrames);
		ScaleKeys[Index].SetNum(CurrentFrames + NewFrames);
	}
	for (int32 Index = 0; Index < NewFrames; ++Index)
	{
		const int32 Frame = Frames[Index];
		TArray<FTransform> BoneTransforms = AnimEval.GetBoneTransforms(Anim, Frame);
		if (BoneTransforms.IsEmpty()) // Frame not found
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Frame %d not found in AnimSequence", Frame);
			return false;
		}
		check (BoneTransforms.Num() == NumBones);
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const FTransform& BoneTransform = BoneTransforms[BoneIndex];
			PosKeys[BoneIndex][CurrentFrames + Index] = (FVector3f(BoneTransform.GetLocation()));
			RotKeys[BoneIndex][CurrentFrames + Index] = (FQuat4f(BoneTransform.GetRotation()));
			ScaleKeys[BoneIndex][CurrentFrames + Index] = FVector3f(BoneTransform.GetScale3D());
		}
	}
	return true;
}

bool UNearestNeighborAnimStream::ToAnim(UAnimSequence* OutAnim) const
{
	if (!IsValid())
	{
		return false;
	}
	OutAnim->SetSkeleton(Skeleton);
	if (PosKeys.IsEmpty())
	{
		return true;
	}
	const int32 NumKeys = PosKeys[0].Num();
	if (NumKeys == 0)
	{
		return true;
	}
	const int32 NumBones = PosKeys.Num();
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		if (PosKeys[BoneIndex].Num() != NumKeys || RotKeys[BoneIndex].Num() != NumKeys || ScaleKeys[BoneIndex].Num() != NumKeys)
		{
			return false;
		}
	}
	IAnimationDataController& Controller = OutAnim->GetController();
	Controller.OpenBracket(LOCTEXT("CreateNewAnim_Bracket", "Create New Anim"));
	Controller.InitializeModel();
	OutAnim->ResetAnimation();
	Controller.SetNumberOfFrames(NumKeys - 1);
	Controller.SetFrameRate(FFrameRate(30, 1));
	const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const FName BoneName = ReferenceSkeleton.GetBoneName(BoneIndex);
		Controller.AddBoneCurve(BoneName);
		Controller.SetBoneTrackKeys(BoneName, PosKeys[BoneIndex], RotKeys[BoneIndex], ScaleKeys[BoneIndex]);
	}

	Controller.NotifyPopulated();
	Controller.CloseBracket();

	return true;
}

void UNearestNeighborGeometryCacheStream::Init(const UGeometryCache* InTemplateCache)
{
	if (!InTemplateCache)
	{
		UE_LOGF(LogNearestNeighborModel, Error, "TemplateCache is None when initializing NearestNeighborGeometryCacheStream");
		return;
	}
	TemplateCache = InTemplateCache;
	TemplateNumTracks = TemplateCache->Tracks.Num();
	TemplateTrackNumVertices.SetNum(TemplateNumTracks);
	for (int32 TrackIndex = 0; TrackIndex < TemplateNumTracks; ++TrackIndex)
	{
		const TObjectPtr<UGeometryCacheTrack> Track = TemplateCache->Tracks[TrackIndex];
		const int32 NumVertices = UE::NearestNeighborModel::Private::GetNumVertices(*Track);
		if (NumVertices == 0)
		{
			UE_LOGF(LogNearestNeighborModel, Warning, "Track %d has no vertices", TrackIndex);
		}
		TemplateTrackNumVertices[TrackIndex] = NumVertices;
	}
	TrackToFrameToPositions.Empty();
	TrackToFrameToPositions.SetNum(TemplateNumTracks);
}

bool UNearestNeighborGeometryCacheStream::IsValid() const
{
	return TemplateCache != nullptr;
}

bool UNearestNeighborGeometryCacheStream::AppendFrames(const UGeometryCache* Cache, TArray<int32> Frames)
{
	if (!Cache || !IsValid())
	{
		return false;
	}
	const int32 NumCacheFrames = UE::NearestNeighborModel::FHelpers::GetNumFrames(Cache);
	const int32 NumTracks = Cache->Tracks.Num();
	if (NumTracks != TemplateNumTracks)
	{
		UE_LOGF(LogNearestNeighborModel, Error, "Number of tracks in Cache (%d) is different from TemplateCache (%d)", NumTracks, TemplateNumTracks);
		return false;
	}
	for (int32 Frame : Frames)
	{
		if (Frame < 0 || Frame >= NumCacheFrames) // Frame not found
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Frame %d not found in GeometryCache", Frame);
			return false;
		}
		for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
		{
			const TObjectPtr<UGeometryCacheTrack> Track = Cache->Tracks[TrackIndex];
			FGeometryCacheMeshData MeshData;
			if (!Track->GetMeshDataAtSampleIndex(Frame, MeshData))
			{
				UE_LOGF(LogNearestNeighborModel, Error, "Frame %d cannot be retrieved in GeometryCache", Frame);
				return false;
			}
			TArray<FVector3f>& Positions = MeshData.Positions;
			if (Positions.Num() != TemplateTrackNumVertices[TrackIndex])
			{
				UE_LOGF(LogNearestNeighborModel, Error, "Number of vertices in Frame %d of Track %d (%d) is different from TemplateCache (%d)", Frame, TrackIndex, Positions.Num(), TemplateTrackNumVertices[TrackIndex]);
				return false;
			}
			TArray<TArray<FVector3f>>& FrameToPositions = TrackToFrameToPositions[TrackIndex];
			FrameToPositions.Add(MoveTemp(Positions));
		}
	}
	return true;
}

bool UNearestNeighborGeometryCacheStream::ToGeometryCache(UGeometryCache* OutCache)
{
	if (!OutCache)
	{
		UE_LOGF(LogNearestNeighborModel, Error, "OutCache is None when converting NearestNeighborGeometryCacheStream to GeometryCache");
		return false;
	}
	if (!IsValid())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "TemplateCache is None when converting NearestNeighborGeometryCacheStream to GeometryCache");
		return false;
	}
	using UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter;
	using UE::GeometryCacheHelpers::AddTrackWritersFromTemplateCache;
	using FTrackWriter = FGeometryCacheConstantTopologyWriter::FTrackWriter;

	FGeometryCacheConstantTopologyWriter Writer(*OutCache);
	const int32 NumAdded = AddTrackWritersFromTemplateCache(Writer, *TemplateCache);
	check(NumAdded == TemplateNumTracks);
	for (int32 TrackIndex = 0; TrackIndex < TemplateNumTracks; ++TrackIndex)
	{
		FTrackWriter& TrackWriter = Writer.GetTrackWriter(TrackIndex);
		TrackWriter.WriteAndClose(TrackToFrameToPositions[TrackIndex]);
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
