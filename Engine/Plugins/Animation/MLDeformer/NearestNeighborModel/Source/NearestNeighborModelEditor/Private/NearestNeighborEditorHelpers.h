// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "NearestNeighborEditorHelpers.generated.h"

#define UE_API NEARESTNEIGHBORMODELEDITOR_API

class UAnimSequence;
UCLASS(MinimalAPI, Blueprintable)
class UNearestNeighborAnimStream : public UObject
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = "Python")
	UE_API void Init(USkeleton* InSkeleton);

	UFUNCTION(BlueprintPure, Category = "Python")
	UE_API bool IsValid() const;

	UFUNCTION(BlueprintCallable, Category = "Python")
	UE_API bool AppendFrames(const UAnimSequence* Anim, TArray<int32> Frames);

	UFUNCTION(BlueprintPure, Category = "Python")
	UE_API bool ToAnim(UAnimSequence* OutAnim) const;

private:
	TObjectPtr<USkeleton> Skeleton;
	TArray<TArray<FVector3f>> PosKeys;
	TArray<TArray<FQuat4f>> RotKeys;
	TArray<TArray<FVector3f>> ScaleKeys;
};

class UGeometryCache;
UCLASS(MinimalAPI, Blueprintable)
class UNearestNeighborGeometryCacheStream : public UObject
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = "Python")
	UE_API void Init(const UGeometryCache* InTemplateCache);

	UFUNCTION(BlueprintPure, Category = "Python")
	UE_API bool IsValid() const;

	UFUNCTION(BlueprintCallable, Category = "Python")
	UE_API bool AppendFrames(const UGeometryCache* Cache, TArray<int32> Frames);

	UFUNCTION(BlueprintPure, Category = "Python")
	UE_API bool ToGeometryCache(UGeometryCache* OutCache);

private:
	const UGeometryCache* TemplateCache = nullptr;
	int32 TemplateNumTracks = 0;
	TArray<int32> TemplateTrackNumVertices;
	TArray<TArray<TArray<FVector3f>>> TrackToFrameToPositions;
};

#undef UE_API
