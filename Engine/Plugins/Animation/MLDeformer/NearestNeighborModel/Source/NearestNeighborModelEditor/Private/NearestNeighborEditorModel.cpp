// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborEditorModel.h"

#include "Algo/Copy.h"
#include "Animation/AnimSequence.h"
#include "Components/ExternalMorphSet.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"
#include "IPersonaPreviewScene.h"
#include "Misc/MessageDialog.h"
#include "MLDeformerAsset.h"
#include "MLDeformerComponent.h"
#include "MLDeformerTrainingModel.h"
#include "MLDeformerEditorStyle.h"
#include "MLDeformerEditorToolkit.h"
#include "NearestNeighborGeomCacheSampler.h"
#include "NearestNeighborModel.h"
#include "NearestNeighborModelInputInfo.h"
#include "NearestNeighborModelInstance.h"
#include "NearestNeighborModelStyle.h"
#include "NearestNeighborModelVizSettings.h"
#include "NearestNeighborTrainingModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "PrimitiveDrawingUtils.h"
#include "SkeletalMeshAttributes.h"

#define LOCTEXT_NAMESPACE "NearestNeighborEditorModel"

namespace UE::NearestNeighborModel
{
	namespace Private
	{
		void AddFloatArrayToDeltaArray(TConstArrayView<float> FloatArr, TConstArrayView<int32> VertexMap, TConstArrayView<float> VertexWeights, TArrayView<FVector3f> OutDeltaArr)
		{
			const int32 SectionNumVerts = VertexMap.Num();
			check(FloatArr.Num() == SectionNumVerts * 3);
			check(VertexWeights.Num() == SectionNumVerts);
			for (int32 Index = 0; Index < SectionNumVerts; ++Index)
			{
				const int32 FloatIndex = Index * 3;
				const int32 DeltaIndex = VertexMap[Index];
				const float Weight = VertexWeights[Index];
				OutDeltaArr[DeltaIndex] = FVector3f(FloatArr[FloatIndex], FloatArr[FloatIndex + 1], FloatArr[FloatIndex + 2]) * Weight;
			}
		}
	};

	UE::MLDeformer::FMLDeformerEditorModel* FNearestNeighborEditorModel::MakeInstance()
	{
		return new FNearestNeighborEditorModel();
	}

	void FNearestNeighborEditorModel::Init(const InitSettings& InitSettings)
	{
		FMLDeformerMorphModelEditorModel::Init(InitSettings);
		InitInputInfo(Model->GetInputInfo());
		VertexMapSelector = MakeUnique<FVertexMapSelector>();
		VertexMapSelector->Update(Model->GetSkeletalMesh());
		UNearestNeighborModel* NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return;
		}

		// Create one section on default.
		if (NearestNeighborModel->GetNumSections() == 0)
		{
			CreateDefaultSection();
		}

		const int32 NumSections = NearestNeighborModel->GetNumSections();
		if (UNearestNeighborModelVizSettings* const NNViz = GetCastVizSettings())
		{
			VertVizSelector = MakeUnique<FVertVizSelector>(NNViz);
			VertVizSelector->Update(NumSections);
		}
	}

	TSharedPtr<UE::MLDeformer::FMLDeformerSampler> FNearestNeighborEditorModel::CreateSamplerObject() const
	{
		return MakeShared<FNearestNeighborGeomCacheSampler>();
	}

	void FNearestNeighborEditorModel::CreateActors(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene)
	{
		FMLDeformerMorphModelEditorModel::CreateActors(InPersonaPreviewScene);
	
		UWorld* const World = InPersonaPreviewScene->GetWorld();
		if (!World)
		{
			return;
		}

		if (NearestNeighborActor)
		{
			EditorActors.Remove(NearestNeighborActor);
			if (NearestNeighborActor->GetActor())
			{
				World->DestroyActor(NearestNeighborActor->GetActor(), true);
			}
		}
		NearestNeighborActor = CreateNearestNeighborActor(World);
		UpdateNearestNeighborActor(*NearestNeighborActor);
		EditorActors.Add(NearestNeighborActor);
	}

	void FNearestNeighborEditorModel::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
	{
		FMLDeformerEditorModel::Tick(ViewportClient, DeltaTime);
		if (NearestNeighborActor)
		{
			UpdateNearestNeighborActor(*NearestNeighborActor);
			NearestNeighborActor->Tick();
		}
		UpdateNearestNeighborIds();
	}

	void FNearestNeighborEditorModel::InitInputInfo(UMLDeformerInputInfo* InputInfo)
	{
		FMLDeformerEditorModel::InitInputInfo(InputInfo);
		UNearestNeighborModelInputInfo* NearestNeighborInputInfo = static_cast<UNearestNeighborModelInputInfo*>(InputInfo);
		NearestNeighborInputInfo->InitRefBoneRotations(Model->GetSkeletalMesh());
	}

	void FNearestNeighborEditorModel::UpdateTrainingDeviceList()
	{
		// Update the list of devices we can use to train with.
		// This is the cpu and list of cuda devices.
		UNearestNeighborTrainingModel* TrainingModel = UE::MLDeformer::NewDerivedObject<UNearestNeighborTrainingModel>();
		if (TrainingModel)
		{
			TrainingModel->Init(this);
			TrainingModel->UpdateAvailableDevices();
			TrainingModel->ConditionalBeginDestroy();
		}
	}
	
	ETrainingResult FNearestNeighborEditorModel::Train()
	{
		UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return ETrainingResult::FailPythonError;
		}
		if(OpFlag::HasError(NearestNeighborModel->UpdateForTraining()))
		{
			return ETrainingResult::FailPythonError;
		}
		if (!NearestNeighborModel->IsReadyForTraining())
		{
			return ETrainingResult::FailPythonError;
		}
		const int32 NumFrames = GetNumFramesForTraining();
		const int32 NumBasis = NearestNeighborModel->GetTotalNumBasis();
		if (NumFrames <= NumBasis)
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Training frames (%d) must be greater than the number of basis (%d)", NumFrames, NumBasis);
			return ETrainingResult::FailPythonError;
		}
		return TrainModel<UNearestNeighborTrainingModel>(this);
	}

	bool FNearestNeighborEditorModel::LoadTrainedNetwork() const
	{
		UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (NearestNeighborModel)
		{	
			const FString File = NearestNeighborModel->GetModelDir() / TEXT("NearestNeighborModel.ubnne");
			const bool bSuccess = NearestNeighborModel->LoadOptimizedNetworkFromFile(File);
			if (bSuccess)
			{
				UNearestNeighborModelInstance* ModelInstance = GetTestNearestNeighborModelInstance();
				if (ModelInstance)
				{
					ModelInstance->InitOptimizedNetworkInstance();
					return true;
				}
			}
		}
		return false;
	}

	void FNearestNeighborEditorModel::OnPostTraining(ETrainingResult TrainingResult, bool bUsePartiallyTrainedWhenAborted)
	{
		if (UNearestNeighborModel* const NearestNeighborModel = GetCastModel())
		{
			if (TrainingResult == ETrainingResult::Success || (TrainingResult == ETrainingResult::Aborted && bUsePartiallyTrainedWhenAborted))
			{
				NearestNeighborModel->InvalidateInference();
				if (NearestNeighborModel->DoesUseFileCache())
				{
					NearestNeighborModel->UpdateFileCache();
				}
			}
			NearestNeighborModel->UpdateForInference();
		}
		FMLDeformerMorphModelEditorModel::OnPostTraining(TrainingResult, bUsePartiallyTrainedWhenAborted);
	}

	FMLDeformerTrainingInputAnim* FNearestNeighborEditorModel::GetTrainingInputAnim(int32 Index) const
	{
		UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return nullptr;
		}
		TArray<FMLDeformerGeomCacheTrainingInputAnim>& TrainingInputAnims = NearestNeighborModel->GetTrainingInputAnims();
		const int32 NumTrainingAnims = TrainingInputAnims.Num();
		const int32 NumSections = NearestNeighborModel->GetNumSections();
		if (Index >= 0 && Index < NumTrainingAnims)
		{
			return &TrainingInputAnims[Index];
		}
		else if (Index >= NumTrainingAnims && Index < NumTrainingAnims + NumSections)
		{
			const int32 SectionIndex = Index - NumTrainingAnims;
			return NearestNeighborModel->GetNearestNeighborAnim(SectionIndex);
		}
		else
		{
			return nullptr;
		}
	}

	void FNearestNeighborEditorModel::UpdateTimelineTrainingAnimList()
	{
		TArray<TSharedPtr<FMLDeformerTrainingInputAnimName>> NameList;
		
		// Build the list of names, based on the training inputs.
		const int32 NumAnims = GetNumTrainingInputAnims();
		for (int32 AnimIndex = 0; AnimIndex < NumAnims; ++AnimIndex)
		{
			const FMLDeformerTrainingInputAnim* Anim = GetTrainingInputAnim(AnimIndex);
			if (Anim)
			{
				if (Anim->IsValid())
				{
					TSharedPtr<FMLDeformerTrainingInputAnimName> AnimName = MakeShared<FMLDeformerTrainingInputAnimName>();
					AnimName->TrainingInputAnimIndex = AnimIndex;
					AnimName->Name = FString::Printf(TEXT("[#%d] %s"), AnimIndex, *Anim->GetAnimSequence()->GetName());
					NameList.Emplace(AnimName);
				}
			}
		}

		if (const UNearestNeighborModel* const NearestNeighborModel = GetCastModel())
		{
			const int32 NumSections = NearestNeighborModel->GetNumSections();
			for (int32 Index = 0; Index < NumSections; ++Index)
			{
				const FMLDeformerTrainingInputAnim* Anim = NearestNeighborModel->GetNearestNeighborAnim(Index);
				if (Anim && Anim->IsValid())
				{
					TSharedPtr<FMLDeformerTrainingInputAnimName> AnimName = MakeShared<FMLDeformerTrainingInputAnimName>();
					AnimName->TrainingInputAnimIndex = Index + NumAnims;
					AnimName->Name = FString::Printf(TEXT("Neighbor[#%d] %s"), Index, *Anim->GetAnimSequence()->GetName());	
					NameList.Emplace(AnimName);
				}
			}
		}

		SetTimelineAnimNames(NameList);
	}

	void FNearestNeighborEditorModel::OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent)
	{
		FMLDeformerMorphModelEditorModel::OnPropertyChanged(PropertyChangedEvent);
		const FProperty* Property = PropertyChangedEvent.Property;
		if (Property == nullptr)
		{
			return;
		}
		UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return;
		}

		if (Property->GetFName() == UMLDeformerModel::GetSkeletalMeshPropertyName())
		{
			if (VertexMapSelector.IsValid())
			{
				VertexMapSelector->Update(NearestNeighborModel->GetSkeletalMesh());
			}
		}

		if (Property->GetFName() == UNearestNeighborModel::GetSectionsPropertyName())
		{
			if (VertVizSelector.IsValid())
			{
				VertVizSelector->Update(NearestNeighborModel->GetNumSections());
			}
			UpdateTimelineTrainingAnimList();
		}

		if (Property->GetFName() == UNearestNeighborModelSection::GetNeighborPosesPropertyName())
		{
			UpdateTimelineTrainingAnimList();
		}
	}

	void FNearestNeighborEditorModel::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
	{
		FMLDeformerMorphModelEditorModel::Render(View, Viewport, PDI);
		const FMLDeformerSampler* Sampler = GetSamplerForActiveAnim();
		if (!Sampler || !Sampler->IsInitialized())
		{
			return;
		}

		const UNearestNeighborModelVizSettings* VizSettings = GetCastVizSettings();
		if (!VizSettings)
		{
			return;
		}

		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
		{
			const TArray<FVector3f>& LinearSkinnedPositions = Sampler->GetSkinnedVertexPositions();
			bool bDrawVerts = VizSettings->bDrawVerts;

			// Disable drawing deltas when we're playing the training anim sequence, as that can get too slow.
			const UE::MLDeformer::FMLDeformerEditorActor* BaseActor = FindEditorActor(UE::MLDeformer::ActorID_Train_Base);
			if (BaseActor)
			{
				bDrawVerts &= !BaseActor->IsPlaying();
			}

			// Draw the verts for the current frame.
			if (bDrawVerts)
			{
				const FLinearColor VertsColor0 = FNearestNeighborModelEditorStyle::Get().GetColor("NearestNeighborModel.Verts.VertsColor0");
				const FLinearColor VertsColor1 = FNearestNeighborModelEditorStyle::Get().GetColor("NearestNeighborModel.Verts.VertsColor1");
				const uint8 DepthGroup = VizSettings->GetXRayDeltas() ? 100 : 0;
				const float PointSize = 5;
				const int32 SectionIndex = VizSettings->VertVizSectionIndex;
				if (SectionIndex != INDEX_NONE)
				{
					const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
					if (NearestNeighborModel)
					{
						TConstArrayView<int32> VertexMap = NearestNeighborModel->GetSection(SectionIndex).GetVertexMap();
						TConstArrayView<float> VertexWeights = NearestNeighborModel->GetSection(SectionIndex).GetVertexWeights();
						for (int32 Index = 0; Index < VertexMap.Num(); ++Index)
						{
							const int32 ArrayIndex = 3 * VertexMap[Index];
							const float Weight = VertexWeights[Index];
							const FLinearColor VertsColor = FMath::Lerp(VertsColor0, VertsColor1, Weight);
							const FVector VertexPos = (FVector)LinearSkinnedPositions[VertexMap[Index]];
							PDI->DrawPoint(VertexPos, VertsColor, PointSize, DepthGroup);
						}
					}
				}
				else // draw all verts
				{
					for (int32 Index = 0; Index < LinearSkinnedPositions.Num(); ++Index)
					{
						const FVector VertexPos = (FVector)LinearSkinnedPositions[Index];
						PDI->DrawPoint(VertexPos, VertsColor1, PointSize, DepthGroup);
					}
				}
			}
		}
	}

	FVertexMapSelector* FNearestNeighborEditorModel::GetVertexMapSelector() const
	{
		return VertexMapSelector.Get();
	}

	FVertVizSelector* FNearestNeighborEditorModel::GetVertVizSelector() const
	{
		return VertVizSelector.Get();
	}

	void FNearestNeighborEditorModel::CreateSamplers()
	{
		FMLDeformerMorphModelEditorModel::CreateSamplers();
		const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return;
		}
		
		const int32 NumTrainingAnims = GetNumTrainingInputAnims();
		const int32 NumSections = NearestNeighborModel->GetNumSections();
		for (int32 Index = 0; Index < NumSections; ++Index)
		{
			TSharedPtr<FNearestNeighborGeomCacheSampler> Sampler = StaticCastSharedPtr<FNearestNeighborGeomCacheSampler>(CreateSamplerObject());
			check(Sampler.IsValid());
			Sampler->Init(this, NumTrainingAnims + Index);
			Samplers.Add(Sampler);
		}
	}

	bool FNearestNeighborEditorModel::IsAnimIndexValid(int32 Index) const
	{
		const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return false;
		}
		const int32 NumSections = NearestNeighborModel->GetNumSections();
		const int32 NumTrainingAnims = GetNumTrainingInputAnims();
		return FMath::IsWithin(Index, 0, NumTrainingAnims + NumSections);
	}

	UNearestNeighborModel* FNearestNeighborEditorModel::GetCastModel() const
	{
		return Cast<UNearestNeighborModel>(Model.Get());
	}

	FNearestNeighborEditorModelActor* FNearestNeighborEditorModel::CreateNearestNeighborActor(UWorld* World) const
	{
		static FLinearColor LabelColor = FNearestNeighborModelEditorStyle::Get().GetColor("NearestNeighborModel.NearestNeighborActors.LabelColor");
		static FLinearColor WireframeColor = FNearestNeighborModelEditorStyle::Get().GetColor("NearestNeighborModel.NearestNeighborActors.WireframeColor");
		static FName ActorName = FName("NearestNeighborActor");
		static FText LabelText = LOCTEXT("NearestNeighborLabelText", "Nearest Neighbor");
		static int32 ActorID = ActorID_NearestNeighborActors;

		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = MakeUniqueObjectName(World, AActor::StaticClass(), ActorName);
		AActor* Actor = World->SpawnActor<AActor>(SpawnParams);
		Actor->SetFlags(RF_Transient);
		
		// Create the Geometry Cache Component.
		UGeometryCacheComponent* GeomCacheComponent = NewObject<UGeometryCacheComponent>(Actor);
		GeomCacheComponent->RegisterComponent();
		GeomCacheComponent->SetOverrideWireframeColor(true);
		GeomCacheComponent->SetWireframeOverrideColor(WireframeColor);
		GeomCacheComponent->MarkRenderStateDirty();
		GeomCacheComponent->SetVisibility(false);
		Actor->SetRootComponent(GeomCacheComponent);
		
		// Create the editor actor.
		UE::MLDeformer::FMLDeformerEditorActor::FConstructSettings Settings;
		Settings.Actor = Actor;
		Settings.TypeID = ActorID;
		Settings.LabelColor = LabelColor;
		Settings.LabelText = LabelText;
		Settings.bIsTrainingActor = false;

		FNearestNeighborEditorModelActor* NewActor = new FNearestNeighborEditorModelActor(Settings);
		NewActor->SetGeometryCacheComponent(GeomCacheComponent);
		return NewActor;
	}

	void FNearestNeighborEditorModel::UpdateNearestNeighborActor(FNearestNeighborEditorModelActor& Actor) const
	{
		using namespace UE::MLDeformer;

		const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return;
		}
		const UNearestNeighborModelVizSettings* NNViz = GetCastVizSettings();
		if (!NNViz)
		{
			return;
		}
		const int32 SectionIndex = NNViz->NearestNeighborActorSectionIndex;
		const int32 NumSections = NearestNeighborModel->GetNumSections();
		if (SectionIndex < 0 || SectionIndex >= NumSections)
		{
			return;
		}

		float MaxOffset = 0.0f;
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor && 
				EditorActor->GetTypeID() != ActorID_NearestNeighborActors &&
				EditorActor->IsVisible()) 
			{
				MaxOffset = FMath::Max(EditorActor->GetMeshOffsetFactor(), MaxOffset);
			}
		}

		Actor.SetMeshOffsetFactor(MaxOffset + 1.0f);

		UGeometryCache* const GeomCache = NearestNeighborModel->GetSection(SectionIndex).GetMutableNeighborMeshes();
		if (!GeomCache)
		{
			return;
		}
		Actor.SetGeometryCache(GeomCache);
		const UMLDeformerComponent* MLDeformerComponent = GetTestMLDeformerComponent();
		if (!MLDeformerComponent)
		{
			return;
		}
		Actor.SetTrackedComponent(MLDeformerComponent, SectionIndex);
	}

	EOpFlag FNearestNeighborEditorModel::Update()
	{
		UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return EOpFlag::Error;
		}
		if (!NearestNeighborModel->IsReadyForTraining())
		{
			if (IsTrained())
			{
				NearestNeighborModel->ClearOptimizedNetwork();
			}
			return EOpFlag::Error;
		}
		EOpFlag UpdateResult = CheckNetwork();
		if (OpFlag::HasError(UpdateResult))
		{
			return UpdateResult;
		}
		if (!IsReadyForTraining())
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Model is not ready for training. Please check if training data is not empty or reload MLDeformer editor.");
			return EOpFlag::Error;
		}
		UpdateResult |= UpdateNearestNeighborData();
		if (OpFlag::HasError(UpdateResult))
		{
			return UpdateResult;
		}
		UpdateResult |= NearestNeighborModel->UpdateForInference();
		if (OpFlag::HasError(UpdateResult))
		{
			return UpdateResult;
		}
		UpdateNearestNeighborActor(*NearestNeighborActor);
		UpdateResult |= UpdateMorphDeltas();
		if (OpFlag::HasError(UpdateResult))
		{
			return UpdateResult;
		}
		InitEngineMorphTargets(NearestNeighborModel->GetMorphTargetDeltas());
		const int32 LOD = 0;
		const TSharedPtr<const FExternalMorphSet> MorphSet = NearestNeighborModel->GetMorphTargetSet(LOD);
		if (MorphSet.IsValid() && MorphSet->MorphBuffers.IsMorphResourcesInitialized())
		{
			NearestNeighborModel->UpdateMorphTargetsLastWriteTime();
		}
		else
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Morph target set is empty");
			UpdateResult |= EOpFlag::Error;
		}
		SetDefaultDeformerGraphIfNeeded();
		return UpdateResult;
	}

	void FNearestNeighborEditorModel::OnUpdateClicked()
	{
		EOpFlag Result = Update();
		if (OpFlag::HasError(Result))
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("UpdateErrorMessage", "Update failed. Please check Output Log for details") , LOCTEXT("UpdateErrorWindowTitle", "Update Results"));
		}
		else if (OpFlag::HasWarning(Result))
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("UpdateWarningMessage", "Update succeeded with warnings. Please check Output Log for details"), LOCTEXT("UpdateWarningWindowTitle", "Update Results"));
		}
	}

	void FNearestNeighborEditorModel::ClearReferences()
	{
		UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (NearestNeighborModel)
		{
			NearestNeighborModel->ClearReferences();
		}
		if (Editor->GetModelDetailsView())
		{
			Editor->GetModelDetailsView()->ForceRefresh();
		}
		if (Editor->GetVizSettingsDetailsView())
		{
			Editor->GetVizSettingsDetailsView()->ForceRefresh();
		}
		UpdateIsReadyForTrainingState();
	}

	EOpFlag FNearestNeighborEditorModel::CheckNetwork()
	{
		const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return EOpFlag::Error;
		}

		if (!NearestNeighborModel->GetOptimizedNetwork().IsValid())
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Network is empty. Please (re-)train model.");
			return EOpFlag::Error;
		}
	
		const int32 NumNetworkWeights = NearestNeighborModel->GetNumNetworkOutputs();
		const int32 NumBasis = NearestNeighborModel->GetTotalNumBasis();
		if (NumNetworkWeights != NumBasis)
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Network output dimension %d does not equal to total number of basis %d. Network needs to be re-trained.", NumNetworkWeights, NumBasis);
			return EOpFlag::Error;
		}

		return EOpFlag::Success;
	}

	EOpFlag FNearestNeighborEditorModel::UpdateNearestNeighborData()
	{
		const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return EOpFlag::Error;
		}
		if (!IsTrained())
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Network is not trained. Nearest neighbor data cannot be updated.");
			return EOpFlag::Error;
		}
		if (NearestNeighborModel->GetNumNetworkOutputs() != NearestNeighborModel->GetTotalNumBasis())
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Network output dimension %d does not equal to total number of basis %d. Please re-train network.", NearestNeighborModel->GetNumNetworkOutputs(), NearestNeighborModel->GetTotalNumBasis());
			return EOpFlag::Error;
		}
		
		UNearestNeighborTrainingModel* TrainingModel = UE::MLDeformer::NewDerivedObject<UNearestNeighborTrainingModel>();
		if (!TrainingModel)
		{
			return EOpFlag::Error;
		}
		TrainingModel->Init(this);

		return ToOpFlag(TrainingModel->UpdateNearestNeighborData());
	}

	EOpFlag FNearestNeighborEditorModel::UpdateMorphDeltas()
	{
		const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return EOpFlag::Error;
		}
		EOpFlag Result = EOpFlag::Success;
	
		const USkeletalMesh* SkelMesh = Model->GetSkeletalMesh();
		const int32 NumBaseMeshVerts = Model->GetNumBaseMeshVerts();
		if (!SkelMesh || NumBaseMeshVerts == 0)
		{
			UE_LOGF(LogNearestNeighborModel, Error, "SkeletalMesh is empty. No morph targets are generated");
			return EOpFlag::Error;
		}
	
		if (Model->GetVertexMap().IsEmpty())
		{
			UE_LOGF(LogNearestNeighborModel, Error, "VertexMap of the skeletal mesh is empty. No morph targets are generated");
			return EOpFlag::Error;
		}
	
		const int32 NumSections = NearestNeighborModel->GetNumSections();
		if (NumSections == 0)
		{
			UE_LOGF(LogNearestNeighborModel, Error, "There are no cloth sections. No morph targets are generated");
			return EOpFlag::Error;
		}
	
		const int32 NumImportedModelVerts = FMath::Max(NearestNeighborModel->GetVertexMap()) + 1;
		if (NumImportedModelVerts != NumBaseMeshVerts)
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Vertex count mismatch: imported model of SkeletalMesh has %d vertices and cached SkeletalMesh has %d vertices", NumImportedModelVerts, NumBaseMeshVerts);
			return EOpFlag::Error;
		}
	
		TArray<FVector3f> Deltas;
	
		const int32 TotalNumBasis = NearestNeighborModel->GetTotalNumBasis();
		const int32 TotalNumNeighbors = NearestNeighborModel->GetTotalNumNeighbors();
		Deltas.SetNumZeroed((1 + TotalNumBasis + TotalNumNeighbors) * NumBaseMeshVerts);
	
		using Private::AddFloatArrayToDeltaArray;
		int32 MorphOffset = 1;
		for (int32 SectionIndex = 0; SectionIndex < NearestNeighborModel->GetNumSections(); ++SectionIndex)
		{
			const FSection& Section = NearestNeighborModel->GetSection(SectionIndex);
			TConstArrayView<int32> VertexMap = Section.GetVertexMap();
			TConstArrayView<float> VertexWeights = Section.GetVertexWeights();
			if (VertexMap.IsEmpty())
			{
				UE_LOGF(LogNearestNeighborModel, Warning, "Section %d has empty vertex map. No morph targets are generated for this section", SectionIndex);
				Result |= EOpFlag::Warning;
				continue;
			}
			check(VertexMap.Num() == Section.GetNumVertices());
			check(VertexWeights.Num() == Section.GetNumVertices());
			TArrayView<FVector3f> MeanDeltas(Deltas.GetData(), NumBaseMeshVerts);
			AddFloatArrayToDeltaArray(Section.GetVertexMean(), VertexMap, VertexWeights, MeanDeltas);
			const int32 NumBasis = Section.GetNumBasis();
			const int32 SectionNumVerts = Section.GetNumVertices();
			check(VertexMap.Num() == SectionNumVerts);
			for (int32 Index = 0; Index < NumBasis; ++Index)
			{
				TConstArrayView<float> BasisFloats(Section.GetBasis().GetData() + Index * SectionNumVerts * 3, SectionNumVerts * 3);
				TArrayView<FVector3f> BasisDeltas(Deltas.GetData() + (MorphOffset + Index) * NumBaseMeshVerts, NumBaseMeshVerts);
				AddFloatArrayToDeltaArray(BasisFloats, VertexMap, VertexWeights, BasisDeltas);
			}
			if (NearestNeighborModel->DoesUsePCA())
			{
				MorphOffset += NumBasis;
			}
		}
		if (!NearestNeighborModel->DoesUsePCA())
		{
			MorphOffset += TotalNumBasis;
		}
	
		for (int32 SectionIndex = 0; SectionIndex < NearestNeighborModel->GetNumSections(); ++SectionIndex)
		{
			const FSection& Section = NearestNeighborModel->GetSection(SectionIndex);
			TConstArrayView<int32> VertexMap = Section.GetVertexMap();
			TConstArrayView<float> VertexWeights = Section.GetVertexWeights();
			if (VertexMap.IsEmpty())
			{
				// Warning already generated
				continue;
			}
			if (!Section.IsReadyForInference())
			{
				UE_LOGF(LogNearestNeighborModel, Warning, "Section %d is not ready for inference. No morph targets are generated for this section", SectionIndex);
				Result |= EOpFlag::Warning;
				continue;
			}
			TArray<float> RuntimeNeighborOffsets;
			Section.GetRuntimeNeighborOffsets(RuntimeNeighborOffsets);
			const int32 SectionNumVerts = Section.GetNumVertices();
			const int32 SectionNumNeighbors = Section.GetRuntimeNumNeighbors();
			check(RuntimeNeighborOffsets.Num() == SectionNumVerts * 3 * SectionNumNeighbors);
			for (int32 Index = 0; Index < SectionNumNeighbors; ++Index)
			{
				TConstArrayView<float> NeighborOffsets(RuntimeNeighborOffsets.GetData() + Index * SectionNumVerts * 3, SectionNumVerts * 3);
				TArrayView<FVector3f> NeighborDeltas(Deltas.GetData() + (MorphOffset + Index) * NumBaseMeshVerts, NumBaseMeshVerts);
				AddFloatArrayToDeltaArray(NeighborOffsets, VertexMap, VertexWeights, NeighborDeltas);
			}
			MorphOffset += SectionNumNeighbors;
		}
	
		if (Deltas.Num() == 0)
		{
			UE_LOGF(LogNearestNeighborModel, Error, "All sections are empty. No morph targets are generated.");
			return EOpFlag::Error;
		}

		GetCastModel()->SetMorphTargetDeltas(Deltas);
		return Result;
	}

	void FNearestNeighborEditorModel::UpdateNearestNeighborIds()
	{
		const UNearestNeighborModelInstance* ModelInstance = GetTestNearestNeighborModelInstance();
		const UNearestNeighborModel* NearestNeighborModel = GetCastModel();
		UNearestNeighborModelVizSettings* NNViz = GetCastVizSettings();
		if (ModelInstance && NearestNeighborModel && NNViz)
		{
			const TArray<uint32>& NeighborIds = ModelInstance->GetNearestNeighborIds();
			const int32 Num = FMath::Min(NeighborIds.Num(), NearestNeighborModel->GetNumSections());
			TArray<int32> AssetNeighborIds;
			AssetNeighborIds.SetNum(Num);
			for(int32 Index = 0; Index < Num; ++Index)
			{
				const int32 NeighborId = NeighborIds[Index];
				const TArray<int32>& IndexMap = NearestNeighborModel->GetSection(Index).GetAssetNeighborIndexMap();
				AssetNeighborIds[Index] = IndexMap.IsValidIndex(NeighborId) ? IndexMap[NeighborId] : INDEX_NONE;
			}
			NNViz->NearestNeighborIds = AssetNeighborIds;
		}
	}

	UNearestNeighborModelVizSettings* FNearestNeighborEditorModel::GetCastVizSettings() const
	{
		return Cast<UNearestNeighborModelVizSettings>(GetMorphModel()->GetVizSettings()); 
	}

	UMLDeformerComponent* FNearestNeighborEditorModel::GetTestMLDeformerComponent() const
	{
		return FindMLDeformerComponent(UE::MLDeformer::ActorID_Test_MLDeformed);
	}

	USkeletalMeshComponent* FNearestNeighborEditorModel::GetTestSkeletalMeshComponent() const
	{
		if (const UMLDeformerComponent* MLDeformerComponent = GetTestMLDeformerComponent())
		{
			return MLDeformerComponent->GetSkeletalMeshComponent();
		}
		return nullptr;
	}
	
	UNearestNeighborModelInstance* FNearestNeighborEditorModel::GetTestNearestNeighborModelInstance() const
	{
		if (const UMLDeformerComponent* MLDeformerComponent = FindMLDeformerComponent(UE::MLDeformer::ActorID_Test_MLDeformed))
		{
			return Cast<UNearestNeighborModelInstance>(MLDeformerComponent->GetModelInstance());
		}
		return nullptr;
	}

	void FNearestNeighborEditorModel::CreateDefaultSection()
	{
		UNearestNeighborModel* NearestNeighborModel = GetCastModel();
		VertexMapSelector->Update(Model->GetSkeletalMesh());
		NearestNeighborModel->AddSection(nullptr);
		FSection* const Section = NearestNeighborModel->OnSectionAdded(0);
		FString MapString;
		if (NearestNeighborModel->GetSkeletalMesh())
		{
			const TArray<FInt32Range> Ranges = NearestNeighborModel->GetMeshVertRanges(*NearestNeighborModel->GetSkeletalMesh());
			const int32 MeshIndex = Section->GetMeshIndex(); 
			if (Ranges.IsValidIndex(MeshIndex))
			{
				MapString = FString::Printf(TEXT("%d-%d"), Ranges[MeshIndex].GetLowerBoundValue(), Ranges[MeshIndex].GetUpperBoundValue() - 1);
			}
		}
		Section->SetVertexMapString(MapString);
		NearestNeighborModel->InvalidateTrainingModelOnly();
		NearestNeighborModel->UpdateNetworkOutputDim();
		NearestNeighborModel->InvalidateTraining();
	}

	void FNearestNeighborEditorModel::CopyBaseSettingsFromModel(const FMLDeformerEditorModel* SourceEditorModel)
	{
		FMLDeformerMorphModelEditorModel::CopyBaseSettingsFromModel(SourceEditorModel);

		UNearestNeighborModel* NearestNeighborModel = GetCastModel();

		// Wipe existing sections (as our init might have created one) and create a default new one.
		NearestNeighborModel->RemoveAllSections();
		CreateDefaultSection();
	}

	void FVertexMapSelector::Update(const USkeletalMesh* SkelMesh)
	{
		constexpr int32 LODIndex = 0;
		if (!SkelMesh || !SkelMesh->HasMeshDescription(LODIndex))
		{
			Reset();
			return;
		}

		const FMeshDescription* MeshDescription = SkelMesh->GetMeshDescription(LODIndex);
		const FSkeletalMeshConstAttributes MeshAttributes(*MeshDescription);
		
		const FSkeletalMeshAttributesShared::FSourceGeometryPartNameConstRef NameRef = MeshAttributes.GetSourceGeometryPartNames();
		const FSkeletalMeshAttributesShared::FSourceGeometryPartVertexOffsetAndCountConstRef PartOffsetAndCountRef = MeshAttributes.GetSourceGeometryPartVertexOffsetAndCounts();
		
		Options.Reserve(MeshAttributes.GetNumSourceGeometryParts() + 1);
		VertexMapStrings.Reserve(MeshAttributes.GetNumSourceGeometryParts() + 1);

		for (const FSourceGeometryPartID GeometryPartID: MeshAttributes.SourceGeometryParts().GetElementIDs())
		{
			FName Name = NameRef.Get(GeometryPartID);
			TArrayView<const int32> OffsetAndCount = PartOffsetAndCountRef.Get(GeometryPartID);

			TSharedPtr<FString> Option = MakeShared<FString>(Name.ToString());
			FString VertexMapString = FString::Printf(TEXT("%d-%d"), OffsetAndCount[0], OffsetAndCount[0] + OffsetAndCount[1] - 1);
			Options.Add(Option);
			VertexMapStrings.Add(Option, MoveTemp(VertexMapString));
		}
		
		Options.Add(CustomString);
		VertexMapStrings.Add(CustomString, FString::Printf(TEXT("0-%d"), MeshDescription->Vertices().Num() - 1));
	}

	TArray<TSharedPtr<FString>>* FVertexMapSelector::GetOptions()
	{
		return &Options;
	}

	void FVertexMapSelector::OnSelectionChanged(UNearestNeighborModelSection& Section, TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo) const
	{
		TSharedPtr<FString> SelectedItem = GetSelectedItem(Section);
		if (InSelectedItem != SelectedItem)
		{
			Section.SetVertexMapString(VertexMapStrings[InSelectedItem]);
			const int32 Index = Options.Find(InSelectedItem);
			if (Index >= 0 && Index < Options.Num() - 1)
			{
				Section.SetMeshIndex(Index);
			}
			else
			{
				Section.SetMeshIndex(INDEX_NONE);
			}
		}
	}

	TSharedPtr<FString> FVertexMapSelector::GetSelectedItem(const UNearestNeighborModelSection& Section) const
	{
		const int32 MeshIndex = Section.GetMeshIndex();
		if (Options.IsValidIndex(MeshIndex))
		{
			return Options[MeshIndex];
		}
		else if (!Options.IsEmpty())
		{
			return Options.Last();
		}
		else
		{
			return nullptr;
		}
	}

	FString FVertexMapSelector::GetVertexMapString(const UNearestNeighborModelSection& Section) const
	{
		const int32 MeshIndex = Section.GetMeshIndex();
		if (Options.IsValidIndex(MeshIndex))
		{
			return VertexMapStrings[Options[MeshIndex]];
		}
		else if (!Options.IsEmpty())
		{
			return VertexMapStrings[Options.Last()];
		}
		else
		{
			return FString();
		}
	}

	bool FVertexMapSelector::IsValid() const
	{
		return Options.Num() > 0 && Options.Num() == VertexMapStrings.Num();
	}

	void FVertexMapSelector::Reset()
	{
		Options.Reset();
		VertexMapStrings.Reset();
	}

	TSharedPtr<FString> FVertexMapSelector::CustomString = MakeShared<FString>(TEXT("Custom"));

	FVertVizSelector::FVertVizSelector(UNearestNeighborModelVizSettings* InSettings)
		: Settings(InSettings)
	{
		Options.Add(MakeShared<FString>(TEXT("All Sections")));
	}

	void FVertVizSelector::Update(int32 NumSections)
	{
		check (NumSections >= 0);
		if (NumSections < Options.Num() - 1)
		{
			Options.SetNum(NumSections + 1);
		}
		else
		{
			for (int32 i = Options.Num() - 1; i < NumSections; ++i)
			{
				Options.Add(MakeShared<FString>(FString::Printf(TEXT("Section %d"), i)));
			}
		}
	}
	
	TArray<TSharedPtr<FString>>* FVertVizSelector::GetOptions()
	{
		return &Options;
	}
	
	void FVertVizSelector::OnSelectionChanged(TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo) const
	{
		if (!Settings)
		{
			return;
		}
		const int32 Index = Options.Find(InSelectedItem);
		if (Index != INDEX_NONE)
		{
			Settings->VertVizSectionIndex = Index - 1;
		}
	}
	
	TSharedPtr<FString> FVertVizSelector::GetSelectedItem() const
	{
		if (!Settings)
		{
			return nullptr;
		}
		return Options[Settings->VertVizSectionIndex + 1];
	}

	int32 FVertVizSelector::GetSectionIndex(TSharedPtr<FString> Item) const
	{
		return Options.Find(Item) - 1;
	}

	void FVertVizSelector::SelectSection(int32 SectionIndex)
	{
		if (!Settings)
		{
			return;
		}
		if (SectionIndex < INDEX_NONE || SectionIndex >= Options.Num() - 1)
		{
			Settings->VertVizSectionIndex = INDEX_NONE; // set to default
		}
		else
		{
			Settings->VertVizSectionIndex = SectionIndex;
		}
	}
}	// namespace UE::NearestNeighborModel

#undef LOCTEXT_NAMESPACE
