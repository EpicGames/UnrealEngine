// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborEditorModelActor.h"

#include "GeometryCacheComponent.h"
#include "MLDeformerComponent.h"
#include "NearestNeighborModelInstance.h"

namespace UE::NearestNeighborModel
{
	FNearestNeighborEditorModelActor::FNearestNeighborEditorModelActor(const FConstructSettings& Settings)
		: FMLDeformerGeomCacheActor(Settings)
	{
	}

	FNearestNeighborEditorModelActor::~FNearestNeighborEditorModelActor() = default;

	void FNearestNeighborEditorModelActor::SetGeometryCache(UGeometryCache* InGeometryCache) const
	{
		if (GeomCacheComponent)
		{
			GeomCacheComponent->SetGeometryCache(InGeometryCache);
		}
	}

	void FNearestNeighborEditorModelActor::SetTrackedComponent(const UMLDeformerComponent* InComponent, int32 InSectionIndex)
	{
		TrackedComponent = InComponent;
		SectionIndex = InSectionIndex;
	}

	void FNearestNeighborEditorModelActor::Tick() const
	{
		if (GeomCacheComponent && GeomCacheComponent->GetGeometryCache() && TrackedComponent.IsValid())
		{
			if (const UNearestNeighborModelInstance* ModelInstance = static_cast<UNearestNeighborModelInstance*>(TrackedComponent->GetModelInstance()))
			{
				const TArray<uint32> NearestNeighborIds = ModelInstance->GetNearestNeighborIds();
				if (NearestNeighborIds.IsValidIndex(SectionIndex))
				{
					GeomCacheComponent->SetManualTick(true);
					GeomCacheComponent->TickAtThisTime(GeomCacheComponent->GetTimeAtFrame(NearestNeighborIds[SectionIndex]), false, false, false);
				}
			}
		}
	}
}	// namespace UE::NearestNeighborModel
