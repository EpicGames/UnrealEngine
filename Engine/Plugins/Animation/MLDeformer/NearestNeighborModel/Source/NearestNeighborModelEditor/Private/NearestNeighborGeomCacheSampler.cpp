// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborGeomCacheSampler.h"

#include "Animation/AnimSequence.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Async/ParallelFor.h"
#include "BoneWeights.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrack.h"
#include "NearestNeighborModel.h"
#include "SkeletalMeshAttributes.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"

namespace UE::NearestNeighborModel
{
	void FNearestNeighborGeomCacheSampler::Sample(int32 InAnimFrameIndex)
	{
		const UNearestNeighborModel* const NearestNeighborModel = static_cast<UNearestNeighborModel*>(Model);
		using UE::MLDeformer::EVertexDeltaSpace;
		if (NearestNeighborModel && NearestNeighborModel->DoesUseDualQuaternionDeltas() && VertexDeltaSpace == EVertexDeltaSpace::PreSkinning)
		{
			FMLDeformerSampler::Sample(InAnimFrameIndex);
			UpdateSkinnedPositions();
			SampleDualQuaternionDeltas(InAnimFrameIndex);
		}
		else
		{
			FMLDeformerGeomCacheSampler::Sample(InAnimFrameIndex);
		}
	}

	namespace Private
	{
		UAnimSequence* GetAnimSequence(const USkeletalMeshComponent* SkeletalMeshComponent)
		{
			if (SkeletalMeshComponent)
			{
				if (UAnimSingleNodeInstance* SingleNodeInstance =  Cast<UAnimSingleNodeInstance>(SkeletalMeshComponent->GetAnimInstance()))
				{
					return Cast<UAnimSequence>(SingleNodeInstance->GetCurrentAsset());
				}
			}
			return nullptr;
		}
	}

	bool FNearestNeighborGeomCacheSampler::CustomSample(int32 Frame)
	{
		if (!SkeletalMeshComponent)
		{
			return false;
		}
		
		UAnimSequence* AnimSequence = Private::GetAnimSequence(SkeletalMeshComponent.Get());
		if (!AnimSequence)
		{
			return false;
		}
		AnimSequence->Interpolation = EAnimInterpolationType::Step;
		const IAnimationDataModel* DataModel = AnimSequence->GetDataModel();
		if (!DataModel)
		{
			return false;
		}
		const int32 NumKeys = DataModel->GetNumberOfKeys();
		if (Frame < 0 || Frame >= NumKeys)
		{
			UE_LOGF(LogNearestNeighborModel, Warning, "AnimSequence only has %d keys, but being sampled with key %d", NumKeys, Frame);
			return false;
		}

		Sample(Frame);
		return true;
	}

	void FNearestNeighborGeomCacheSampler::Customize(UAnimSequence* Anim, UGeometryCache* Cache)
	{
		const TObjectPtr<USkeletalMeshComponent> SkelMeshComp = GetSkeletalMeshComponent();
		if (SkelMeshComp)
		{
			SkelMeshComp->SetAnimationMode(EAnimationMode::AnimationSingleNode);
			SkelMeshComp->SetAnimation(Anim);
			SkelMeshComp->SetPosition(0.0f);
			SkelMeshComp->SetPlayRate(1.0f);
			SkelMeshComp->Play(false);
			SkelMeshComp->RefreshBoneTransforms();
			if (SkelMeshComp->GetAnimInstance())
			{
				SkelMeshComp->GetAnimInstance()->GetRequiredBones().SetUseRAWData(true);
			}
		}

		const TObjectPtr<UGeometryCacheComponent> GeomCacheComp = GetGeometryCacheComponent();
		if (GeomCacheComp)
		{
			GeomCacheComp->SetGeometryCache(Cache);
			GeomCacheComp->ResetAnimationTime();
			GeomCacheComp->SetLooping(false);
			GeomCacheComp->SetManualTick(true);
			GeomCacheComp->SetPlaybackSpeed(1.0f);
			GeomCacheComp->Play();
		}

		if (Anim && SkelMeshComp && Cache && GeomCacheComp)
		{
			GenerateMeshMappings();
		}
	}

	void FNearestNeighborGeomCacheSampler::SampleDualQuaternionDeltas(int32 InAnimFrameIndex)
	{
		USkeletalMesh* const SkeletalMesh = SkeletalMeshComponent.Get() ? SkeletalMeshComponent->GetSkeletalMeshAsset() : nullptr;
		UGeometryCache* const GeometryCache = GeometryCacheComponent->GetGeometryCache();
		if (!SkeletalMeshComponent || !SkeletalMesh || !GeometryCacheComponent || !GeometryCache)
		{
			VertexDeltas.Reset(0);
			return;
		}
		const FTransform& AlignmentTransform = Model->GetAlignmentTransform();
		constexpr int32 LODIndex = 0;
		if (!SkeletalMesh->HasMeshDescription(LODIndex))
		{
			VertexDeltas.Reset(0);
			return;
		}
		const FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
		const FSkinWeightVertexBuffer* SkinWeightBuffer = SkeletalMeshComponent->GetSkinWeightBuffer(LODIndex);
		if (!RenderData || !RenderData->LODRenderData.IsValidIndex(LODIndex) || !SkinWeightBuffer)
		{
			VertexDeltas.Reset(0);
			return;
		}
		const FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[LODIndex];
		const FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LODIndex);
		const FSkeletalMeshConstAttributes MeshAttributes(*MeshDescription);
		const FSkeletalMeshAttributesShared::FSourceGeometryPartVertexOffsetAndCountConstRef PartOffsetAndCountRef = MeshAttributes.GetSourceGeometryPartVertexOffsetAndCounts();
		
		if (GeomCacheMeshDatas.Num() != MeshMappings.Num())
		{
			GeomCacheMeshDatas.SetNum(MeshMappings.Num());
		}

		// For all mesh mappings we found.
		for (int32 MeshMappingIndex = 0; MeshMappingIndex < MeshMappings.Num(); ++MeshMappingIndex)
		{
			const UE::MLDeformer::FMLDeformerGeomCacheMeshMapping& MeshMapping = MeshMappings[MeshMappingIndex];
			UGeometryCacheTrack* const Track = GeometryCache->Tracks[MeshMapping.TrackIndex];
		
			// Sample the mesh data of the geom cache.
			FGeometryCacheMeshData& GeomCacheMeshData = GeomCacheMeshDatas[MeshMappingIndex];
			if (!Track->GetMeshDataAtSampleIndex(InAnimFrameIndex, GeomCacheMeshData))
			{
				continue;
			}

			TArrayView<const int32> OffsetAndCount = PartOffsetAndCountRef.Get(MeshMapping.MeshIndex);
			const int32 VertexOffset = OffsetAndCount[0];
			const int32 NumVertices = OffsetAndCount[1];
			
			constexpr int32 BatchSize = 500;
			const int32 NumBatches = (NumVertices / BatchSize) + 1;
			ParallelFor(NumBatches, [&](int32 BatchIndex)
			{
				const int32 StartVertex = BatchIndex * BatchSize;
				if (StartVertex >= NumVertices || VertexDeltas.IsEmpty())
				{
					return;
				}
				const int32 NumVertsInBatch = (StartVertex + BatchSize) < NumVertices ? BatchSize : FMath::Max(NumVertices - StartVertex, 0);
				for (int32 VertexIndex = StartVertex; VertexIndex < StartVertex + NumVertsInBatch; ++VertexIndex)
				{
					const int32 SkinnedVertexIndex = VertexOffset + VertexIndex;
					const int32 GeomCacheVertexIndex = MeshMapping.SkelMeshToTrackVertexMap[VertexIndex];
					if (GeomCacheMeshData.Positions.IsValidIndex(GeomCacheVertexIndex))
					{
						FVector3f Delta = FVector3f::ZeroVector;
						const int32 ArrayIndex = 3 * SkinnedVertexIndex;
						const int32 RenderVertexIndex = MeshMapping.ImportedVertexToRenderVertexMap[VertexIndex];
						if (RenderVertexIndex != INDEX_NONE)
						{
							const FVector3f SkinnedVertexPos = SkinnedVertexPositions[SkinnedVertexIndex];
							const FVector3f GeomCacheVertexPos = (FVector3f)AlignmentTransform.TransformPosition((FVector)GeomCacheMeshData.Positions[GeomCacheVertexIndex]);
							const FVector3f WorldDelta = GeomCacheVertexPos - SkinnedVertexPos;
							Delta = CalcDualQuaternionDelta(RenderVertexIndex, WorldDelta, LODRenderData, *SkinWeightBuffer);
						}
						VertexDeltas[ArrayIndex] = Delta.X;
						VertexDeltas[ArrayIndex + 1] = Delta.Y;
						VertexDeltas[ArrayIndex + 2] = Delta.Z;
					}
				}
			});
		}
	}

	namespace Private
	{
		template<typename QuatType>
		QuatType Conjugate(const QuatType& Q)
		{
			return QuatType(-Q.X, -Q.Y, -Q.Z, Q.W);
		}
		
		template<typename QuatType>
		float Inner(const QuatType& Q1, const QuatType& Q2)
		{
			return Q1.X * Q2.X + Q1.Y * Q2.Y + Q1.Z * Q2.Z + Q1.W * Q2.W;
		}
		
		template<typename QuatType>
		QuatType FromVector(const FVector3f& V)
		{
			return QuatType(V[0], V[1], V[2], 0);
		}
		
		template<typename QuatType>
		FVector3f ToVector(const QuatType& Q)
		{
			return FVector3f(Q.X, Q.Y, Q.Z);
		}
	};

	FVector3f FNearestNeighborGeomCacheSampler::CalcDualQuaternionDelta(int32 VertexIndex, const FVector3f& WorldDelta, const FSkeletalMeshLODRenderData& SkelMeshLODData, const FSkinWeightVertexBuffer& SkinWeightBuffer) const
	{
		using namespace Private;
		check(SkeletalMeshComponent);
		const USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		check(Mesh);
		const FSkeletalMeshRenderData* RenderData = Mesh->GetResourceForRendering();
		constexpr int32 LODIndex = 0;
		check(RenderData && RenderData->LODRenderData.IsValidIndex(LODIndex));
		const FSkeletalMeshLODRenderData& LODData = RenderData->LODRenderData[LODIndex];

		// Find the render section, which we need to find the right bone index.
		int32 SectionIndex = INDEX_NONE;
		int32 SectionVertexIndex = INDEX_NONE;
		LODData.GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);

		FQuat4f QuatSum = FQuat4f(0, 0, 0, 0);

		FQuat4f R0;
		float Sign = 1;
		const int32 NumInfluences = SkinWeightBuffer.GetMaxBoneInfluences();
		for (int32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; ++InfluenceIndex)
		{
			const int32 BoneIndex = SkinWeightBuffer.GetBoneIndex(VertexIndex, InfluenceIndex);
			const uint16 WeightByte = SkinWeightBuffer.GetBoneWeight(VertexIndex, InfluenceIndex);
			// Weight must be > 0 when InflueceIndex == 0
			ensure(InfluenceIndex > 0 || WeightByte > 0);
			if (WeightByte > 0)
			{
				const int32 RealBoneIndex = LODData.RenderSections[SectionIndex].BoneMap[BoneIndex];
				const FQuat4f R(BoneMatrices[RealBoneIndex].GetMatrixWithoutScale());
				if (InfluenceIndex == 0)
				{
					R0 = R; Sign = 1;
				}
				else
				{
					Sign = Inner(R0, R) < 0 ? -1 : 1;
				}
				const float Weight = Sign * static_cast<float>(WeightByte) * UE::AnimationCore::InvMaxRawBoneWeightFloat;
				QuatSum += R * Weight;
			}
		}

		const float SizeSquared = QuatSum.SizeSquared();
		if (SizeSquared > SMALL_NUMBER)
		{
			/** Unrotate vector using v' = q^{-1} v q if q is unit size 
			 * Because QuatSum is not unit size
			 * v' =  q^* v q / |q|^2
			 */
			return ToVector<FQuat4f>(Conjugate(QuatSum) * FromVector<FQuat4f>(WorldDelta) * QuatSum) / SizeSquared;
		}
		else
		{
			return WorldDelta;			
		}
	}

	EOpFlag FNearestNeighborGeomCacheSampler::GenerateMeshMappings()
	{
		EOpFlag Result = EOpFlag::Success;
		USkeletalMesh* const SkeletalMesh = SkeletalMeshComponent.Get() ? SkeletalMeshComponent->GetSkeletalMeshAsset() : nullptr;
		UGeometryCache* const GeometryCache = GeometryCacheComponent.Get() ? GeometryCacheComponent->GetGeometryCache() : nullptr;
		if (SkeletalMeshComponent && SkeletalMesh && GeometryCacheComponent && GeometryCache)
		{
			TArray<FString> FailedNames;
			TArray<FString> VertexMisMatchNames;
			GenerateGeomCacheMeshMappings(SkeletalMesh, GeometryCache, MeshMappings, FailedNames, VertexMisMatchNames);
			if (!FailedNames.IsEmpty() || !VertexMisMatchNames.IsEmpty())
			{
				Result |= EOpFlag::Warning;
			}
			for(int32 i = 0; i < VertexMisMatchNames.Num(); i++)
			{
				UE_LOGF(LogNearestNeighborModel, Warning, "%ls is skipped because it has different vertex counts in skeletal mesh and geometry cache.", *VertexMisMatchNames[i]);
			}
			GeomCacheMeshDatas.Reset();
			GeomCacheMeshDatas.AddDefaulted(MeshMappings.Num());
			
			Result |= CheckMeshMappingsEmpty();
		}
		return Result;
	}

	EOpFlag FNearestNeighborGeomCacheSampler::CheckMeshMappingsEmpty() const
	{
		if(MeshMappings.IsEmpty())
		{
			UE_LOGF(LogNearestNeighborModel, Error, "MeshMappings is empty. Unable to match skeletal mesh with geometry cache.");
			return EOpFlag::Error;
		}
		else
		{
			return EOpFlag::Success;
		}
	}

	TArray<uint32> FNearestNeighborGeomCacheSampler::GetMeshIndexBuffer() const
	{
		TArray<uint32> IndexBuffer;

		if (SkeletalMeshComponent == nullptr)
		{
			return IndexBuffer;
		}

		const USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (Mesh == nullptr)
		{
			return IndexBuffer;
		}

		const FSkeletalMeshRenderData* RenderData = Mesh->GetResourceForRendering();
		constexpr int32 LODIndex = 0;
		if(!RenderData || !RenderData->LODRenderData.IsValidIndex(LODIndex))
		{
			return IndexBuffer;
		}
		const FSkeletalMeshLODRenderData& LODRenderData = Mesh->GetResourceForRendering()->LODRenderData[LODIndex];

		const FSkeletalMeshModel* SkeletalMeshModel = Mesh->GetImportedModel();
		if (!SkeletalMeshModel || !SkeletalMeshModel->LODModels.IsValidIndex(LODIndex))
		{
			return IndexBuffer;
		}
		const FSkeletalMeshLODModel& LODModel = SkeletalMeshModel->LODModels[LODIndex];

		LODRenderData.MultiSizeIndexContainer.GetIndexBuffer(IndexBuffer);
		TConstArrayView<int32> ImportedVertexNumbers {LODModel.MeshToImportVertexMap};
		
		if (ImportedVertexNumbers.Num() > 0)
		{
			for (int32 Index = 0; Index < IndexBuffer.Num(); Index++)
			{
				IndexBuffer[Index] = ImportedVertexNumbers[IndexBuffer[Index]];
			}
		}

		return IndexBuffer;
	}
};
