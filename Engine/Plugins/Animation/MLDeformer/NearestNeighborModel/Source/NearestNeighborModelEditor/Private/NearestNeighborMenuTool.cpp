// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborMenuTool.h"
#include "IDocumentation.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerEditorToolkit.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

namespace UE::NearestNeighborModel
{
	namespace Private::MenuTool
	{
		class SToolWidget : public SCompoundWidget 
		{
			SLATE_BEGIN_ARGS(SToolWidget) 
				{}
				SLATE_ARGUMENT(TObjectPtr<UObject>, Data)
			SLATE_END_ARGS()
		
		public:
			void Construct(const FArguments& InArgs);
			UObject* GetData() const;
		
		private:
			TSharedPtr<IDetailsView> DetailsView;
			TStrongObjectPtr<UObject> Data;
		};

		void SToolWidget::Construct(const FArguments& InArgs)
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

			FDetailsViewArgs Args;
			Args.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea;
			Args.bAllowSearch = false;
			Args.bShowObjectLabel = false;
			DetailsView = PropertyModule.CreateDetailView(Args);
			Data = TStrongObjectPtr<UObject>(InArgs._Data.Get());
			DetailsView->SetObject(Data.Get());

		
			this->ChildSlot
			[
				DetailsView.ToSharedRef()
			];
		}
		
		UObject* SToolWidget::GetData() const
		{
			return Data.Get();
		}

		class FTabSummoner : public FWorkflowTabFactory
		{
		public:
			using FMLDeformerEditorToolkit = ::UE::MLDeformer::FMLDeformerEditorToolkit;
		
			FTabSummoner(TSharedRef<FNearestNeighborMenuTool> InTool, const TSharedRef<FMLDeformerEditorToolkit>& InEditor)
				: FWorkflowTabFactory(InTool->GetToolName(), InEditor)
				, Tool(InTool)
			{
				bIsSingleton = true;
				TabLabel = FText::FromName(Tool->GetToolName());
				ViewMenuDescription = Tool->GetToolTip();
				ViewMenuTooltip = Tool->GetToolTip();
			}

			~FTabSummoner() override = default;

			virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
			{
				UObject* Data = Tool->CreateData();
				if (!Data)
				{
					return SNullWidget::NullWidget;
				}
				TSharedPtr<FMLDeformerEditorToolkit> Toolkit = StaticCastSharedPtr<FMLDeformerEditorToolkit>(HostingApp.Pin());
				if (!Toolkit.IsValid())
				{
					return SNullWidget::NullWidget;
				}
				TSharedRef<SToolWidget> Widget = SNew(SToolWidget)
					.Data(Data);

				Tool->InitData(*Data, *Toolkit);

				using UE::MLDeformer::FMLDeformerEditorModel;
				TSharedPtr<FMLDeformerEditorModel> EditorModel = Toolkit->GetActiveModelPointer().Pin(); 
				if (!EditorModel.IsValid())
				{
					return SNullWidget::NullWidget;
				}
				
				return SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					[
						Widget
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						Tool->CreateAdditionalWidgets(*Data, EditorModel)
					];
			}

			virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override
			{
				return IDocumentation::Get()->CreateToolTip(
					Tool->GetToolTip(),
					NULL,
					TEXT("Shared/Editors/Persona"),
					Tool->GetToolName().ToString() + "_Window");
			}
		private:
			TSharedRef<FNearestNeighborMenuTool> Tool;
		};
		
		class FMenuExtender : public UE::MLDeformer::FToolsMenuExtender
		{
		public:
			using FMLDeformerEditorToolkit = UE::MLDeformer::FMLDeformerEditorToolkit;
			FMenuExtender(TSharedRef<FNearestNeighborMenuTool> InTool)
				: Tool(InTool)
			{
			}
			~FMenuExtender() override = default;
			virtual FMenuEntryParams GetMenuEntry(FMLDeformerEditorToolkit& Toolkit) const override
			{
				FMenuEntryParams Params;
				Params.DirectActions = FUIAction(
					FExecuteAction::CreateLambda([this, &Toolkit]()
					{
						Toolkit.GetAssociatedTabManager()->TryInvokeTab(Tool->GetToolName());
					}),
					FCanExecuteAction::CreateLambda([]()
					{
						return true;
					})
				);
				Params.LabelOverride = FText::FromName(Tool->GetToolName());
				Params.ToolTipOverride = Tool->GetToolTip();
				return Params;
			}

			virtual TSharedPtr<FWorkflowTabFactory> GetTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& Toolkit) const
			{
				return MakeShared<FTabSummoner>(Tool, Toolkit);
			}

		private:
			TSharedRef<FNearestNeighborMenuTool> Tool;
		};
	}

	void FNearestNeighborMenuTool::Register()
	{
		UE::MLDeformer::FMLDeformerEditorToolkit::AddToolsMenuExtender(MakeUnique<Private::MenuTool::FMenuExtender>(SharedThis(this)));
	}

	UObject* FNearestNeighborMenuTool::CreateData()
	{
		return nullptr;
	}

	void FNearestNeighborMenuTool::InitData(UObject& Data, UE::MLDeformer::FMLDeformerEditorToolkit& Toolkit)
	{
	}

	TSharedRef<SWidget> FNearestNeighborMenuTool::CreateAdditionalWidgets(UObject& Data, TWeakPtr<UE::MLDeformer::FMLDeformerEditorModel> EditorModel)
	{
		return SNullWidget::NullWidget;
	}
};
