// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::MLDeformer
{
	class FMLDeformerEditorModel;
	class FMLDeformerEditorToolkit;
};

namespace UE::NearestNeighborModel
{
	class FNearestNeighborMenuTool : public TSharedFromThis<FNearestNeighborMenuTool>
	{
	public:
		FNearestNeighborMenuTool() = default;
		virtual ~FNearestNeighborMenuTool() = default;

		void Register();

		virtual FName GetToolName() = 0;
		virtual FText GetToolTip() = 0;
		virtual UObject* CreateData();
		virtual void InitData(UObject& Data, UE::MLDeformer::FMLDeformerEditorToolkit& Toolkit);
		// Capture Widget.Data or Widget.EditorModel is not safe because they can be changed after CreateAdditionalWidgets is called. Capture Widget instead.
		virtual TSharedRef<SWidget> CreateAdditionalWidgets(UObject& Data, TWeakPtr<UE::MLDeformer::FMLDeformerEditorModel> EditorModel);
	};
}	// namespace UE::NearestNeighborModel