// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborModelDetails.h"

#include "Components/ExternalMorphSet.h"
#include "DesktopPlatformModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "NearestNeighborEditorModel.h"
#include "NearestNeighborModel.h"
#include "NearestNeighborModelSectionCustomization.h"
#include "PropertyCustomizationHelpers.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "NearestNeighborModelDetails"

namespace UE::NearestNeighborModel	
{

	static bool bShowFileCacheCategory = false;
	FAutoConsoleVariableRef CVarNearestNeighborShowFileCacheCategory(TEXT("MLDeformer.NearestNeighbor.ShowFileCacheCategory"), bShowFileCacheCategory, TEXT("Whether or not to show the file cache category in the NearestNeighbor details panel"));


	TSharedRef<IDetailCustomization> FNearestNeighborModelDetails::MakeInstance()
	{
		return MakeShareable(new FNearestNeighborModelDetails());
	}

	bool FNearestNeighborModelDetails::UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects)
	{
		if (!FMLDeformerMorphModelDetails::UpdateMemberPointers(Objects))
		{
			return false;
		}

		return (GetCastModel() && GetCastEditorModel());
	}

	void FNearestNeighborModelDetails::CreateCategories()
	{
		FMLDeformerMorphModelDetails::CreateCategories();

		NearestNeighborCategoryBuilder = &DetailLayoutBuilder->EditCategory("Nearest Neighbor Setting", FText::GetEmpty(), ECategoryPriority::Important);
		SectionsCategoryBuilder = &DetailLayoutBuilder->EditCategory("Sections", FText::GetEmpty(), ECategoryPriority::Important);
		StatusCategoryBuilder = &DetailLayoutBuilder->EditCategory("Status", FText::GetEmpty(), ECategoryPriority::Important);
	}

	UNearestNeighborModel* FNearestNeighborModelDetails::GetCastModel() const
	{
		return Cast<UNearestNeighborModel>(Model);
	}

	FNearestNeighborEditorModel* FNearestNeighborModelDetails::GetCastEditorModel() const
	{
		return static_cast<FNearestNeighborEditorModel*>(EditorModel);
	}

	namespace Private
	{
		DECLARE_DELEGATE_RetVal(FText, FGetTextDelegate);
		void AddTextRow(IDetailCategoryBuilder& CategoryBuilder, const FText& FilterString, const FText& Name, const FGetTextDelegate ValueDelegate)
		{
			check(ValueDelegate.IsBound());
			CategoryBuilder.AddCustomRow(FilterString)
				.NameContent()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(2, 2)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(Name)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
				.ValueContent()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(2, 2)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text_Lambda([ValueDelegate]
						{
							return ValueDelegate.Execute();
						})
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				];
		}

		void AddFileCacheRow(IDetailCategoryBuilder& CategoryBuilder, const FText& FilterString, const FText& Name, const FGetTextDelegate ValueDelegate, TSharedRef<SWidget>& ValueWidget, TAttribute<bool> bIsEnabled, TAttribute<EVisibility> IsVisible)
		{
			check(ValueDelegate.IsBound());
			CategoryBuilder.AddCustomRow(FilterString)
				.NameContent()
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(2, 2)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
								.Text(Name)
								.Font(IDetailLayoutBuilder::GetDetailFont())
						]
				]
				.ValueContent()
				.MinDesiredWidth(300.f)
				.MaxDesiredWidth(300.f)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(2, 2)
						.VAlign(VAlign_Center)
						.FillWidth(0.6f)
						[
							SNew(STextBlock)
								.Text_Lambda([ValueDelegate]
									{
										return ValueDelegate.Execute();
									})
								.Font(IDetailLayoutBuilder::GetDetailFont())
						]
						+ SHorizontalBox::Slot()
						.FillWidth(0.4f)
						.VAlign(VAlign_Center)
						[
							ValueWidget
						]
				]
				.IsEnabled(bIsEnabled)
				.Visibility(IsVisible);
		}

		FDateTime ToLocalTime(const FDateTime& UtcTime)
		{
			const FDateTime UtcNow = FDateTime::UtcNow();
			const FDateTime Now = FDateTime::Now();
			const FTimespan TimeZoneOffset = Now - UtcNow;
			return UtcTime + TimeZoneOffset;
		}

		FString ToString(const FDateTime& Time)
		{
			return Time.ToString(TEXT("%Y-%m-%d %H:%M:%S"));
		}

		TOptional<FString> OpenDialogAndSelectFolder(const FString& StartingDirectory)
		{
			TOptional<FString> None;
			IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
			if (!DesktopPlatform)
			{
				return None;
			}

			FString FolderName;
			bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			    FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			    TEXT("Choose a directory"),
			    StartingDirectory,
			    FolderName
			);

			if (bFolderSelected)
			{
				return FolderName;
			}
			return None;
		}

		bool DeleteFile(const FString& FilePath)
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (PlatformFile.FileExists(*FilePath))
			{
				return PlatformFile.DeleteFile(*FilePath);
			}
			return false; 
		}
	};

	void FNearestNeighborModelDetails::CustomizeTrainingSettingsCategory() const
	{
		if (!TrainingSettingsCategoryBuilder)
		{
			return;
		}
		TrainingSettingsCategoryBuilder->AddProperty(UNearestNeighborModel::GetInputDimPropertyName());
		TrainingSettingsCategoryBuilder->AddProperty(UNearestNeighborModel::GetHiddenLayerDimsPropertyName());
		TrainingSettingsCategoryBuilder->AddProperty(UNearestNeighborModel::GetOutputDimPropertyName());
		TrainingSettingsCategoryBuilder->AddProperty(UNearestNeighborModel::GetNumIterationsPropertyName());
		TrainingSettingsCategoryBuilder->AddProperty(UNearestNeighborModel::GetBatchSizePropertyName());
		TrainingSettingsCategoryBuilder->AddProperty(UNearestNeighborModel::GetLearningRatePropertyName());
	}

	void FNearestNeighborModelDetails::CustomizeNearestNeighborSettingsCategory() const
	{
		if (!NearestNeighborCategoryBuilder)
		{
			return;
		}
		NearestNeighborCategoryBuilder->AddProperty(UNearestNeighborModel::GetUsePCAPropertyName());
		NearestNeighborCategoryBuilder->AddProperty(UNearestNeighborModel::GetNumBasisPerSectionPropertyName());
		NearestNeighborCategoryBuilder->AddProperty(UNearestNeighborModel::GetUseDualQuaternionDeltasPropertyName());
		NearestNeighborCategoryBuilder->AddProperty(UNearestNeighborModel::GetDecayFactorPropertyName());
		NearestNeighborCategoryBuilder->AddProperty(UNearestNeighborModel::GetNearestNeighborOffsetWeightPropertyName());
		NearestNeighborCategoryBuilder->AddProperty(UNearestNeighborModel::GetUseRBFPropertyName());
		NearestNeighborCategoryBuilder->AddProperty(UNearestNeighborModel::GetRBFSigmaPropertyName());
	}

	void FNearestNeighborModelDetails::CustomizeStatusCategory() const
	{
		if (!StatusCategoryBuilder)
		{
			return;
		}
		Private::AddTextRow(*StatusCategoryBuilder, 
			LOCTEXT("NetworkStateInfo", "Network State"), 
			LOCTEXT("NetworkStateName", "Network State"),
			Private::FGetTextDelegate::CreateLambda([this]
			{
				const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
				if (NearestNeighborModel && NearestNeighborModel->GetOptimizedNetwork().IsValid())
				{
					if (NearestNeighborModel->IsBeforeCustomVersionWasAdded())
					{
						return FText::FromString(TEXT("Loaded from previous version."));
					}
					else
					{
						const FDateTime LocalTime = Private::ToLocalTime(NearestNeighborModel->GetNetworkLastWriteTime());
						return FText::FromString(FString::Printf(TEXT("Last written: %s"), *Private::ToString(LocalTime)));
					}
				}
				else
				{
					return FText::FromString(TEXT("Empty"));
				}
			}));
		
		Private::AddTextRow(*StatusCategoryBuilder,
			LOCTEXT("NetworkArchitectureInfo", "Network Architecture"),
			LOCTEXT("NetworkArchitectureName", "Network Architecture"),
			Private::FGetTextDelegate::CreateLambda([this]
			{
				const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
				if (NearestNeighborModel && NearestNeighborModel->GetOptimizedNetwork().IsValid())
				{
					if (NearestNeighborModel->IsBeforeCustomVersionWasAdded())
					{
						return FText::FromString(TEXT("Loaded from previous version."));
					}
					else
					{
						return FText::FromString(FString::Printf(TEXT("Last written: %s"),  *NearestNeighborModel->GetNetworkLastWriteArchitectureString()));
					}
				}
				else
				{
					return FText::FromString(TEXT("None"));
				}
			}));
		
		Private::AddTextRow(*StatusCategoryBuilder, 
			LOCTEXT("MorphTargetsStateInfo", "Morph Targets State"), 
			LOCTEXT("MorphTargetsStateName", "Morph Targets State"),
			Private::FGetTextDelegate::CreateLambda([this]
			{
				const int32 LOD = 0;
				if (const UNearestNeighborModel* const NearestNeighborModel = GetCastModel())
				{
					if (NearestNeighborModel->GetNumLODs() > 0)
					{
						const TSharedPtr<const FExternalMorphSet> MorphSet = NearestNeighborModel->GetMorphTargetSet(LOD);
						if (MorphSet.IsValid() && MorphSet->MorphBuffers.IsMorphResourcesInitialized())
						{
							if (NearestNeighborModel->IsBeforeCustomVersionWasAdded())
							{
								return FText::FromString(TEXT("Loaded from previous version."));
							}
							else
							{
								const FDateTime Time = NearestNeighborModel->GetMorphTargetsLastWriteTime();
								if (Time != FDateTime::MinValue())
								{
									const FDateTime LocalTime = Private::ToLocalTime(Time);
									return FText::FromString(FString::Printf(TEXT("Last written: %s"), *Private::ToString(LocalTime)));
								}
							}
						}
					}
				}
				return FText::FromString(TEXT("Empty"));
			}));
		
		Private::AddTextRow(*StatusCategoryBuilder, 
			LOCTEXT("InferenceStateInfo", "Inference State"),
			LOCTEXT("InferenceStateName", "Inference State"),
			Private::FGetTextDelegate::CreateLambda([this]
			{
				const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
				if (NearestNeighborModel)
				{
					if (NearestNeighborModel->IsReadyForInference())
					{
						return FText::FromString(TEXT("Valid"));
					}
					else if (NearestNeighborModel->IsReadyForTraining() && NearestNeighborModel->CanDynamicallyUpdateMorphTargets())
					{
						return FText::FromString(TEXT("Invalid, please Update"));
					}
					else
					{
						return FText::FromString(TEXT("Invalid, please Train Model"));
					}
				}
				else
				{
					return FText::FromString(TEXT("Invalid"));
				}
			}));
		
		StatusCategoryBuilder->AddCustomRow(FText::FromString("StatusUpdate")).WholeRowContent()
		[
			SNew(SBox)
			.Padding(2, 2)
			.MaxDesiredWidth(200.f)
			[
				SNew(SButton)
				.Text_Lambda([this]
				{
					const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
					if (NearestNeighborModel && NearestNeighborModel->IsReadyForInference())
					{
						return FText::FromString(TEXT("Update"));
					}
					else
					{
						return FText::FromString(TEXT("Update*"));
					}
				})
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]
				{
					if (GetCastEditorModel())
					{
						GetCastEditorModel()->OnUpdateClicked();
					}
					return FReply::Handled();
				})
				.IsEnabled_Lambda([this]
				{
					const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
					if (NearestNeighborModel)
					{
						return NearestNeighborModel->IsReadyForTraining() && NearestNeighborModel->CanDynamicallyUpdateMorphTargets();
					}
					else
					{
						return false;
					}
				})
			]
		];

		StatusCategoryBuilder->AddCustomRow(FText::FromString("StatusUpdate")).WholeRowContent()
		[
			SNew(SBox)
			.Padding(2, 2)
			.MaxDesiredWidth(200.f)
			[
				SNew(SButton)
				.Text(FText::FromString(TEXT("Clear References")))
				.ToolTipText(LOCTEXT("ClearReferencesTooltip", "Clears all animation sequences and geometry caches for asset validation."))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]
				{
					if (GetCastEditorModel())
					{
						GetCastEditorModel()->ClearReferences();
					}
					return FReply::Handled();
				})
			]
		];
	}

	void FNearestNeighborModelDetails::CustomizeSectionsCategory(IDetailLayoutBuilder& DetailBuilder)
	{
		TSharedRef<IPropertyHandle> SectionsPropertyHandle = DetailBuilder.GetProperty(UNearestNeighborModel::GetSectionsPropertyName());
		UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (SectionsPropertyHandle->AsArray().IsValid() && NearestNeighborModel)
		{
			TSharedRef<FDetailArrayBuilder> PropertyBuilder = MakeShared<FDetailArrayBuilder>(SectionsPropertyHandle, true, false, true);
			PropertyBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FNearestNeighborModelDetails::GenerateSectionElementWidget));
			SectionsCategoryBuilder->AddCustomBuilder(PropertyBuilder);
		}
	}

	void FNearestNeighborModelDetails::CustomizeMorphTargetCategory(IDetailLayoutBuilder& DetailBuilder) const
	{
		DetailBuilder.HideProperty(UNearestNeighborModel::GetIncludeMorphTargetNormalsPropertyName());
	}

	void FNearestNeighborModelDetails::CustomizeFileCacheCategory(IDetailLayoutBuilder& DetailBuilder) const
	{
		IDetailCategoryBuilder& Builder = DetailLayoutBuilder->EditCategory("File Cache", FText::GetEmpty(), ECategoryPriority::Important);

		TAttribute<EVisibility> IsVisible = TAttribute<EVisibility>::Create([this]
			{
				UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
				return ((NearestNeighborModel && NearestNeighborModel->DoesUseFileCache()) || bShowFileCacheCategory) ? EVisibility::Visible : EVisibility::Collapsed;
			});


		Builder.AddProperty(UNearestNeighborModel::GetUseFileCachePropertyName())
			.Visibility(IsVisible);

		Builder.AddCustomRow(FText::FromString("FileCacheWarning"))
		.Visibility(IsVisible)
		.WholeRowContent()
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FileCacheWarning", "Caution: cache files now need to be manually deleted otherwise there can be unexpected results."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.AutoWrapText(true)
			]
		];
		TSharedPtr<IPropertyHandle> DirectoryHandle = DetailBuilder.GetProperty(UNearestNeighborModel::GetFileCacheDirectoryPropertyName());
		Builder.AddProperty(DirectoryHandle).CustomWidget()
		.NameContent()
		[
			DirectoryHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(0.6f)
			[
				DirectoryHandle->CreatePropertyValueWidget()
			]
			+ SHorizontalBox::Slot()
			.FillWidth(0.4f)
			[
				SNew(SButton)
				.Text(LOCTEXT("SelectFileCacheFolder", "Select"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]()
				{
					UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
					if (!NearestNeighborModel)
					{
						return FReply::Handled();
					}
					TOptional<FString> FolderName = Private::OpenDialogAndSelectFolder(NearestNeighborModel->GetFileCacheDirectory());
					if (FolderName.IsSet())
					{
						NearestNeighborModel->SetFileCacheDirectory(FolderName.GetValue());
						NearestNeighborModel->UpdateFileCache();
					}
					return FReply::Handled();
				})
			]
		];

		UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
		if (!NearestNeighborModel)
		{
			return;
		}

		auto CreateTextDelegate = [NearestNeighborModel](TFunction<TOptional<FDateTime>()> TimestampFunc)
		{
			return Private::FGetTextDelegate::CreateLambda([TimestampFunc]
			{
				const TOptional<FDateTime> Timestamp = TimestampFunc();
				if (Timestamp.IsSet())
				{
					return FText::FromString(Private::ToString(Private::ToLocalTime(Timestamp.GetValue())));
				}
				else
				{
					return FText::FromString(TEXT("None"));
				}
			});
		};

		auto CreateDeleteButton = [NearestNeighborModel](TFunction<TArray<FString>()> PathsFunc)
		{
			return StaticCastSharedRef<SWidget>(SNew(SButton)
					.Text(LOCTEXT("DeleteFileCache", "Delete"))
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([NearestNeighborModel, PathsFunc]()
					{
						const TArray<FString> Paths = PathsFunc();
						for (const FString& Path : Paths)
						{
							Private::DeleteFile(Path);
						}
						NearestNeighborModel->UpdateFileCache();
						return FReply::Handled();
					}));
		};

		TAttribute<bool> bIsEnabled = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([NearestNeighborModel]
		{
			return NearestNeighborModel->DoesUseFileCache();
		}));

		{
			TSharedRef<SWidget> Widget = CreateDeleteButton([NearestNeighborModel]()
			{
				return NearestNeighborModel->GetCachedDeltasPaths();
			});
			Private::AddFileCacheRow(Builder,
				LOCTEXT("CachedDeltasInfo", "Cached Deltas"),
				LOCTEXT("CachedDeltasName", "Cached Deltas"),
				CreateTextDelegate([NearestNeighborModel]
				{
					return NearestNeighborModel->GetCachedDeltasTimestamp();
				}),
				Widget,
				bIsEnabled,
				IsVisible);
		}
		{
			TSharedRef<SWidget> Widget = CreateDeleteButton([NearestNeighborModel]()
			{
				return NearestNeighborModel->GetCachedPCAPaths();
			});
			Private::AddFileCacheRow(Builder,
				LOCTEXT("CachedPCAInfo", "Cached PCA"),
				LOCTEXT("CachedPCAName", "Cached PCA"),
				CreateTextDelegate([NearestNeighborModel]
				{
					return NearestNeighborModel->GetCachedPCATimestamp();
				}),
				Widget,
				bIsEnabled,
				IsVisible);
		}
		{
			TSharedRef<SWidget> Widget = CreateDeleteButton([NearestNeighborModel]()
			{
				return NearestNeighborModel->GetCachedNetworkPaths();
			});
			Private::AddFileCacheRow(Builder,
				LOCTEXT("CachedNetworkInfo", "Cached Network"),
				LOCTEXT("CachedNetworkName", "Cached Network"),
				CreateTextDelegate([NearestNeighborModel]
				{
					return NearestNeighborModel->GetCachedNetworkTimestamp();
				}),
				Widget,
				bIsEnabled,
				IsVisible);
		}

		Builder.AddCustomRow(FText::FromString("FileCacheRefresh")).WholeRowContent()
		[
			SNew(SBox)
			.MaxDesiredWidth(200.f)
			.Padding(2, 2)
			[
				SNew(SButton)
				.Text(LOCTEXT("RefreshFileCache", "Refresh"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([NearestNeighborModel]
				{
					NearestNeighborModel->UpdateFileCache();
					return FReply::Handled();
				})
				.IsEnabled(bIsEnabled)
				.Visibility(IsVisible)
			]
		];
	}

	void FNearestNeighborModelDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		// Create all the detail categories and add the properties of the base class.
		FMLDeformerMorphModelDetails::CustomizeDetails(DetailBuilder);

		// Show a warning about this model being deprecated.
		FDetailWidgetRow& GeomCacheMappingErrorRow = BaseMeshCategoryBuilder->AddCustomRow(FText::FromString("ModelDeprecationWarning"))
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Warning)
					.Message(LOCTEXT("ModelDeprecationWarningText", "The Nearest Neighbor Model has been deprecated and is being replaced by the Detail Pose Model. Please use that model instead."))
				]
			];

		CustomizeTrainingSettingsCategory();
		CustomizeNearestNeighborSettingsCategory();
		CustomizeSectionsCategory(DetailBuilder);
		CustomizeMorphTargetCategory(DetailBuilder);
		CustomizeStatusCategory();
		CustomizeFileCacheCategory(DetailBuilder);
	}

	void FNearestNeighborModelDetails::GenerateSectionElementWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
	{
		if (ArrayIndex < GetCastModel()->GetNumSections())
		{
			UNearestNeighborModelSection& Section = GetCastModel()->GetSection(ArrayIndex);
			ChildrenBuilder.AddProperty(PropertyHandle).CustomWidget()
			.OverrideResetToDefault(FResetToDefaultOverride::Hide())
			.WholeRowContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(SNearestNeighborModelSectionWidget)
					.Section(&Section)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(4.0f, 1.0f, 0.0f, 1.0f)
				[ 
					PropertyCustomizationHelpers::MakeInsertDeleteDuplicateButton(
						FExecuteAction::CreateLambda([PropertyHandle]()
						{
							TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->GetParentHandle()->AsArray();
							check(ArrayHandle.IsValid());
							const int32 Index = PropertyHandle->GetArrayIndex();
							ArrayHandle->Insert(Index);
						}),
						FExecuteAction::CreateLambda([PropertyHandle]()
						{
							TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->GetParentHandle()->AsArray();
							check(ArrayHandle.IsValid());
							const int32 Index = PropertyHandle->GetArrayIndex();
							ArrayHandle->DeleteItem(Index);
						}),
						FExecuteAction::CreateLambda([PropertyHandle]()
						{
							TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->GetParentHandle()->AsArray();
							check(ArrayHandle.IsValid());
							const int32 Index = PropertyHandle->GetArrayIndex();
							ArrayHandle->DuplicateItem(Index);
						}))
				]
			];
		}
	}
}	// namespace UE::NearestNeighborModel

#undef LOCTEXT_NAMESPACE
