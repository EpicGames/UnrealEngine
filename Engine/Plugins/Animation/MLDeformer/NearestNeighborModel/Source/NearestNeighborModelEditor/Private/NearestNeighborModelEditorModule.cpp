// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorModeRegistry.h"
#include "MLDeformerEditorModule.h"
#include "Modules/ModuleManager.h"
#include "NearestNeighborEditorModel.h"
#include "NearestNeighborEditorModelActor.h"
#include "NearestNeighborModel.h"
#include "NearestNeighborModelSectionCustomization.h"
#include "NearestNeighborModelDetails.h"
#include "NearestNeighborModelVizSettingsDetails.h"
#include "Tools/NearestNeighborKMeansTool.h"
#include "Tools/NearestNeighborStatsTool.h"

#define LOCTEXT_NAMESPACE "NearestNeighborModelEditorModule"

namespace UE::NearestNeighborModel
{
	class FNearestNeighborModelEditorModule
		: public IModuleInterface
	{
	public:
		// IModuleInterface overrides.
		void StartupModule() override;
		void ShutdownModule() override;
		// ~END IModuleInterface overrides.
	private:
		TSharedPtr<FNearestNeighborKMeansTool> KMeansTool;
		TSharedPtr<FNearestNeighborStatsTool> StatsTool;
	};
}
IMPLEMENT_MODULE(UE::NearestNeighborModel::FNearestNeighborModelEditorModule, NearestNeighborModelEditor)

namespace UE::NearestNeighborModel
{
	void FNearestNeighborModelEditorModule::StartupModule()
	{
		// Register object detail customizations.
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout("NearestNeighborModelVizSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FNearestNeighborModelVizSettingsDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("NearestNeighborModel", FOnGetDetailCustomizationInstance::CreateStatic(&FNearestNeighborModelDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("NearestNeighborModelSection", FOnGetDetailCustomizationInstance::CreateStatic(&FNearestNeighborModelSectionCustomization::MakeInstance));
		PropertyModule.NotifyCustomizationModuleChanged();

		// Register our custom ML deformer model to the model registry in the ML Deformer Framework.
		using ::UE::MLDeformer::FMLDeformerEditorModule;
		using ::UE::MLDeformer::FMLDeformerEditorModelRegistry;
		using ::UE::MLDeformer::FOnGetEditorModelInstance;
		using ::UE::MLDeformer::FMLDeformerEditorModelRegistry;
		FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();
		ModelRegistry.RegisterEditorModel(UNearestNeighborModel::StaticClass(), FOnGetEditorModelInstance::CreateStatic(&FNearestNeighborEditorModel::MakeInstance));
		KMeansTool = MakeShared<FNearestNeighborKMeansTool>();
		KMeansTool->Register();
		StatsTool = MakeShared<FNearestNeighborStatsTool>();
		StatsTool->Register();
	}

	void FNearestNeighborModelEditorModule::ShutdownModule()
	{
		// Unregister our ML Deformer model.
		if (FModuleManager::Get().IsModuleLoaded(TEXT("MLDeformerFrameworkEditor")))
		{
			using ::UE::MLDeformer::FMLDeformerEditorModule;
			using ::UE::MLDeformer::FMLDeformerEditorModelRegistry;
			FMLDeformerEditorModule& EditorModule = FModuleManager::GetModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
			FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();
			ModelRegistry.UnregisterEditorModel(UNearestNeighborModel::StaticClass());
		}

		// Unregister object detail customizations for this model.
		if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
			PropertyModule.UnregisterCustomClassLayout(TEXT("NearestNeighborModelVizSettings"));
			PropertyModule.UnregisterCustomClassLayout(TEXT("NearestNeighborModel"));
			PropertyModule.UnregisterCustomClassLayout(TEXT("NearestNeighborModelSection"));
			PropertyModule.NotifyCustomizationModuleChanged();
		}
	}

}	// namespace UE::NearestNeighborModel

#undef LOCTEXT_NAMESPACE
