// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborModelSectionCustomization.h"

#include "DesktopPlatformModule.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "Engine/SkeletalMesh.h"
#include "MLDeformerEditorModule.h"
#include "MLDeformerEditorStyle.h"
#include "NearestNeighborModel.h"
#include "NearestNeighborEditorModel.h"
#include "SkeletalMeshAttributes.h"
#include "Rendering/SkeletalMeshLODImporterData.h" 
#include "Rendering/SkeletalMeshModel.h"
#include "SMLDeformerBonePickerDialog.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "NearestNeighborModelSectionCustomization"

namespace UE::NearestNeighborModel
{
	namespace Private
	{
		FNearestNeighborEditorModel* GetEditorModel(const UNearestNeighborModel* Model)
		{
			if (!Model)
			{
				return nullptr;
			}
			using ::UE::MLDeformer::FMLDeformerEditorModule;
			FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
			return static_cast<FNearestNeighborEditorModel*>(EditorModule.GetModelRegistry().GetEditorModel(const_cast<UNearestNeighborModel*>(Model)));
		}

		void CreateVertexAttributes(USkeletalMesh& SkeletalMesh, const FString& AttributeName, const TArray<int32>& VertexMap, const TArray<float>& VertexWeights)
		{
			constexpr int32 LODIndex = 0;

			FMeshDescription* MeshDescription = SkeletalMesh.GetMeshDescription(LODIndex);
			if (!MeshDescription)
			{
				return;
			}

			FSkeletalMeshAttributes MeshAttributes(*MeshDescription);

			if (!MeshDescription->VertexAttributes().HasAttribute(*AttributeName))
			{
				MeshDescription->VertexAttributes().RegisterAttribute<float>(*AttributeName, 1, 0.0f);
			}
			TVertexAttributesRef<float> AttributeRef = MeshDescription->VertexAttributes().GetAttributesRef<float>(*AttributeName);

			check(VertexMap.Num() == VertexWeights.Num());
			for (int32 Index = 0; Index < VertexMap.Num(); ++Index)
			{
				AttributeRef.Set(VertexMap[Index], VertexWeights[Index]);
			}
			
			SkeletalMesh.CommitMeshDescription(LODIndex);
		}


		class SNewAttributesDialog
			: public SCustomDialog
		{
		public:
			SLATE_BEGIN_ARGS(SNewAttributesDialog) {}
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs);
			const FString& GetAttributesName() const { return AttributesName; }
		private:
			FString AttributesName;
		};

		void SNewAttributesDialog::Construct(const FArguments& InArgs)
		{
			FText DialogTitle = LOCTEXT("EnterAttributesNameDialogTitle", "New Vertex Attributes");
			SCustomDialog::Construct
			(
				SCustomDialog::FArguments()
				.AutoCloseOnButtonPress(true)
				.Title(DialogTitle)
				.UseScrollBox(false)
				.Buttons(
				{
					SCustomDialog::FButton(LOCTEXT("OKText", "OK"))
					.SetPrimary(true)
					.SetFocus(),
					SCustomDialog::FButton(LOCTEXT("CancelText", "Cancel"))
				})
				.Content()
				[
					SNew(SBox)
					.Padding(FMargin(10.0f, 10.0f))
					.MinDesiredWidth(400.0f)
					.HAlign(EHorizontalAlignment::HAlign_Fill)
					.VAlign(EVerticalAlignment::VAlign_Fill)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(EVerticalAlignment::VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AttributesName", "Attributes Name:"))
						]
						+ SHorizontalBox::Slot()
						.VAlign(EVerticalAlignment::VAlign_Center)
						[
							SNew(SBox)
							.MinDesiredWidth(200.0f)
							.Padding(FMargin(4.0f, 0.0f))
							[
								SNew(SEditableTextBox)
								.OnTextChanged_Lambda([&AttributesName = AttributesName](const FText& InText)
								{
									AttributesName = InText.ToString();
								})
							]
						]
					]
				]
			);
		}

		void CreateWeightMapWidgetSelectedBones(FDetailWidgetRow& Row, IDetailLayoutBuilder& LayoutBuilder, UNearestNeighborModelSection& Section, const FNearestNeighborEditorModel& EditorModel)
		{
			Row.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SelectedBones", "Selected Bones"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(4, 2)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text_Lambda([&Section]()
					{
						return FText::FromString(Section.GetBoneNamesString());
					})
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText_Lambda([&Section]()
					{
						return FText::FromString(Section.GetBoneNamesString());
					})
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.Text(LOCTEXT("SelectBones", "Select Bones"))
						.OnClicked_Lambda([&Section, &EditorModel]()
						{
							UNearestNeighborModel* const Model = Cast<UNearestNeighborModel>(EditorModel.GetModel());
							if (!Model)
							{
								return FReply::Handled();
							}
							USkeletalMesh* const SkelMesh = Model->GetSkeletalMesh();
							if (!SkelMesh)
							{
								return FReply::Handled();
							}
							const FLinearColor HighlightColor = UE::MLDeformer::FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.HighlightColor");
							TSharedPtr<UE::MLDeformer::SMLDeformerBonePickerDialog> Dialog = 
								SNew(UE::MLDeformer::SMLDeformerBonePickerDialog)
								.RefSkeleton(&SkelMesh->GetRefSkeleton())
								.AllowMultiSelect(true)
								.HighlightBoneNamesColor(FSlateColor(HighlightColor))
								.HighlightBoneNames(Section.GetBoneNames());
							
							Dialog->ShowModal();
							const TArray<FName>& BoneNames = Dialog->GetPickedBoneNames();
							if (!BoneNames.IsEmpty())
							{
								Section.SetBoneNames(BoneNames);
							}
							return FReply::Handled();
						})
					]
					+ SHorizontalBox::Slot()
					[
						SNew(SButton)
						.Text(LOCTEXT("CreateAttributes", "Create Attributes"))
						.OnClicked_Lambda([&Section, &EditorModel]()
						{
							TSharedPtr<SNewAttributesDialog> Dialog = SNew(SNewAttributesDialog);
							Dialog->ShowModal();
							const FString& AttributesName = Dialog->GetAttributesName();
							UE_LOGF(LogNearestNeighborModel, Log, "Create Attributes: %ls", *AttributesName);

							UNearestNeighborModel* const Model = Cast<UNearestNeighborModel>(EditorModel.GetModel());
							if (!Model || !Model->GetSkeletalMesh())
							{
								return FReply::Handled();
							}

							CreateVertexAttributes(*Model->GetSkeletalMesh(), AttributesName, Section.GetVertexMap(), Section.GetVertexWeights());
							return FReply::Handled();
						})
					]
				]
			];
		}

		void CreateWeightMapWidgetFromText(FDetailWidgetRow& Row, IDetailLayoutBuilder& LayoutBuilder,UNearestNeighborModelSection& Section, const FNearestNeighborEditorModel& EditorModel)
		{
			TSharedPtr<IPropertyHandle> VertexMapHandle = LayoutBuilder.GetProperty(UNearestNeighborModelSection::GetVertexMapStringPropertyName());
			if (VertexMapHandle.IsValid())
			{
				Row.NameContent()
				[
					VertexMapHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(2, 2)
					.AutoHeight()
					[
						SNew(STextComboBox)
						.OptionsSource(EditorModel.GetVertexMapSelector()->GetOptions())
						.OnSelectionChanged_Lambda([&Section, &EditorModel](TSharedPtr<FString> InString, ESelectInfo::Type InSelectInfo)
							{
								EditorModel.GetVertexMapSelector()->OnSelectionChanged(Section, InString, InSelectInfo);
							})
						.InitiallySelectedItem(EditorModel.GetVertexMapSelector()->GetSelectedItem(Section))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([&Section]()
						{
							return Section.GetMeshIndex() == INDEX_NONE;
						})))
						[
							VertexMapHandle->CreatePropertyValueWidget()
						]
					]
				];
				LayoutBuilder.HideProperty(VertexMapHandle);
			}
		}

		TOptional<FString> OpenTxtFileDialog(const FString& DefaultPath)
		{
			TOptional<FString> Empty;
			IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
			if (DesktopPlatform == nullptr)
			{
				return Empty;
			}

			TArray<FString> FileTypes;
			FileTypes.Add("Text Files (*.txt)|*.txt");

			TArray<FString> OutFileNames;
			const bool bFileDialogOpened = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				TEXT("Choose a txt"),
				DefaultPath,
				TEXT(""),
				FileTypes[0],
				EFileDialogFlags::None,
				OutFileNames
			);

			if (bFileDialogOpened && OutFileNames.Num() > 0)
			{
				return OutFileNames[0];
			}

			return Empty;
		}
	};

	void SNearestNeighborModelSectionWidget::Construct(const FArguments& InArgs)
	{
		Section = InArgs._Section;
	
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
		FDetailsViewArgs Args;
		Args.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea;
		Args.bAllowSearch = false;
		Args.bShowObjectLabel = false;
		Args.bShowScrollBar = false;
		DetailsView = PropertyModule.CreateDetailView(Args);
		DetailsView->SetObject(Section);
	
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				DetailsView.ToSharedRef()
			]
		];
	}

	void FNearestNeighborModelSectionCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		DetailBuilder.HideCategory("Section Private");
		IDetailCategoryBuilder& SectionBuilder = DetailBuilder.EditCategory("Section", LOCTEXT("SectionCategory", "Section"));
		SectionBuilder.AddProperty(UNearestNeighborModelSection::GetNumBasisPropertyName());

		// Get the selected objects
		const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DetailBuilder.GetSelectedObjects();
		if (SelectedObjects.IsEmpty())
		{
			return;
		}
		TWeakObjectPtr<UObject> SelectedObject = SelectedObjects[0];
		if (!SelectedObject.IsValid())
		{
			return;
		}
		UNearestNeighborModelSection* Section = Cast<UNearestNeighborModelSection>(SelectedObject.Get());
		if (!Section)
		{
			return;
		}

		const FNearestNeighborEditorModel* EditorModel = Private::GetEditorModel(Section->GetModel());
		if (!EditorModel)
		{
			return;
		}

		SectionBuilder.AddProperty(UNearestNeighborModelSection::GetWeightMapCreationMethodPropertyName());

		FDetailWidgetRow& WeightMapFromTextRow = SectionBuilder.AddCustomRow(LOCTEXT("WeightMapFromText", "WeightMapFromText"))
			.OverrideResetToDefault(FResetToDefaultOverride::Hide())
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([Section]()
			{
				return Section->GetWeightMapCreationMethod() == 
				ENearestNeighborModelSectionWeightMapCreationMethod::FromText 
				? EVisibility::Visible : EVisibility::Collapsed;
			})));
		Private::CreateWeightMapWidgetFromText(WeightMapFromTextRow, DetailBuilder, *Section, *EditorModel);

		FDetailWidgetRow& WeightMapSelectedBonesRow = SectionBuilder.AddCustomRow(LOCTEXT("WeightMapSelectedBones", "WeightMapSelectedBones"))
			.OverrideResetToDefault(FResetToDefaultOverride::Hide())
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([Section]()
			{
				return Section->GetWeightMapCreationMethod() == 
				ENearestNeighborModelSectionWeightMapCreationMethod::SelectedBones 
				? EVisibility::Visible : EVisibility::Collapsed;
			})));
		Private::CreateWeightMapWidgetSelectedBones(WeightMapSelectedBonesRow, DetailBuilder, *Section, *EditorModel);

		SectionBuilder.AddProperty(UNearestNeighborModelSection::GetAttributeNamePropertyName())
			.OverrideResetToDefault(FResetToDefaultOverride::Hide())
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([Section]()
			{
				return Section->GetWeightMapCreationMethod() == 
				ENearestNeighborModelSectionWeightMapCreationMethod::VertexAttributes 
				? EVisibility::Visible : EVisibility::Collapsed;
			})));

		
		IDetailPropertyRow& ExternalTxtFileRow = SectionBuilder.AddProperty(UNearestNeighborModelSection::GetExternalTxtFilePropertyName())
			.OverrideResetToDefault(FResetToDefaultOverride::Hide())
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([Section]()
			{
				return Section->GetWeightMapCreationMethod() == 
				ENearestNeighborModelSectionWeightMapCreationMethod::ExternalTxt 
				? EVisibility::Visible : EVisibility::Collapsed;
			})));
		
		ExternalTxtFileRow.CustomWidget()
		.NameContent()
		[
			ExternalTxtFileRow.GetPropertyHandle()->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(0.8f)
			.VAlign(VAlign_Center)
			[
				ExternalTxtFileRow.GetPropertyHandle()->CreatePropertyValueWidget()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.Padding(2, 2)
				.MinDesiredWidth(20.f)
				[
					SNew(SButton)
					.Text(LOCTEXT("ChooseExternalTxtButton", "..."))
					.OnClicked_Lambda([Section]()
					{
						using ::UE::NearestNeighborModel::Private::OpenTxtFileDialog;
						const FString CurrentPath = Section->GetExternalTxtFile();
						FString DefaultPath = FPaths::ProjectIntermediateDir();
						if (CurrentPath.Len() > 0)
						{
							DefaultPath = FPaths::GetPath(CurrentPath);
						}
						TOptional<FString> PathOpt = OpenTxtFileDialog(DefaultPath);
						if (PathOpt.IsSet())
						{
							Section->SetExternalTxtFile(PathOpt.GetValue());
						}
						return FReply::Handled();
					})
				]
			]
		];

		SectionBuilder.AddProperty(UNearestNeighborModelSection::GetNeighborPosesPropertyName());
		SectionBuilder.AddProperty(UNearestNeighborModelSection::GetNeighborMeshesPropertyName());
		SectionBuilder.AddProperty(UNearestNeighborModelSection::GetExcludedFramesPropertyName());
	}
}	// namespace UE::NearestNeighborModel

#undef LOCTEXT_NAMESPACE