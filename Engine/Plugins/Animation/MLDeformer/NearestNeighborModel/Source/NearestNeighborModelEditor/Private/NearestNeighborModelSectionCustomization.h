// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Widgets/SCompoundWidget.h"

class IDetailsView;
class UNearestNeighborModelSection;
namespace UE::NearestNeighborModel
{
	class SNearestNeighborModelSectionWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SNearestNeighborModelSectionWidget) {}
			SLATE_ARGUMENT(UNearestNeighborModelSection*, Section)
		SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs);
	
	private:
		TObjectPtr<UNearestNeighborModelSection> Section;
		TSharedPtr<IDetailsView> DetailsView;
	};

	class FNearestNeighborModelSectionCustomization : public IDetailCustomization
	{
	public:
		static TSharedRef<IDetailCustomization> MakeInstance()
		{
			return MakeShared<FNearestNeighborModelSectionCustomization>();
		}

		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	};
};
