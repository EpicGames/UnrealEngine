// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborModelStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Math/Color.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"

namespace UE::NearestNeighborModel
{
	#define MLDEFORMER_IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)

	FNearestNeighborModelEditorStyle::FNearestNeighborModelEditorStyle() : FSlateStyleSet("NearestNeighborModelEditorStyle")
	{
		const FString ResourceDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("NearestNeighborModel"))->GetBaseDir(), TEXT("Resources"));
		SetContentRoot(ResourceDir);

		// Colors and sizes.
		Set("NearestNeighborModel.NearestNeighborActors.WireframeColor", FLinearColor(1.0f, 1.0f, 0.0f));
		Set("NearestNeighborModel.NearestNeighborActors.LabelColor", FLinearColor(1.0f, 1.0f, 0.0f));
		Set("NearestNeighborModel.Verts.VertsColor0", FLinearColor(0.7f, 0.7f, 0.7f));
		Set("NearestNeighborModel.Verts.VertsColor1", FLinearColor(1.0f, 0.5f, 0.0f));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FNearestNeighborModelEditorStyle::~FNearestNeighborModelEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	FNearestNeighborModelEditorStyle& FNearestNeighborModelEditorStyle::Get()
	{
		static FNearestNeighborModelEditorStyle Inst;
		return Inst;
	}
}	// namespace UE::NearestNeighborModel

#undef MLDEFORMER_IMAGE_BRUSH
