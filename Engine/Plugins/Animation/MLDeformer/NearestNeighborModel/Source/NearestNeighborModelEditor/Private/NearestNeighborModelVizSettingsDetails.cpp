// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborModelVizSettingsDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "NearestNeighborEditorModel.h"
#include "NearestNeighborModelVizSettings.h"
#include "Widgets/Input/STextComboBox.h"

#define LOCTEXT_NAMESPACE "MLDeformerMorphModelVizSettingsDetails"

namespace UE::NearestNeighborModel
{
	TSharedRef<IDetailCustomization> FNearestNeighborModelVizSettingsDetails::MakeInstance()
	{
		return MakeShareable(new FNearestNeighborModelVizSettingsDetails());
	}

	void FNearestNeighborModelVizSettingsDetails::AddAdditionalSettings()
	{
		FMLDeformerMorphModelVizSettingsDetails::AddAdditionalSettings();
		AddTrainingMeshAdditionalSettings();
		AddLiveAdditionalSettings();
	}

	void FNearestNeighborModelVizSettingsDetails::AddTrainingMeshAdditionalSettings()
	{
		if (!DetailLayoutBuilder || !TrainingMeshesCategoryBuilder)
		{
			return;
		}
		const FNearestNeighborEditorModel* const NNEditorModel = GetCastEditorModel();
		if (!EditorModel)
		{
			return;
		}

		TrainingMeshesCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UNearestNeighborModelVizSettings, bDrawVerts), UNearestNeighborModelVizSettings::StaticClass());

		FVertVizSelector* const VertVizSelector = NNEditorModel->GetVertVizSelector();
		if (!VertVizSelector)
		{
			return;
		}

		TSharedRef<STextComboBox> VertVizComboBox = SNew(STextComboBox)
			.OptionsSource(VertVizSelector->GetOptions())
			.InitiallySelectedItem(VertVizSelector->GetSelectedItem())
			.OnSelectionChanged_Lambda([VertVizSelector](TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo)
			{
				VertVizSelector->OnSelectionChanged(InSelectedItem, SelectInfo);
			})
			.Font(IDetailLayoutBuilder::GetDetailFont());

		TSharedRef<IPropertyHandle> VertVizHandle = DetailLayoutBuilder->GetProperty(GET_MEMBER_NAME_CHECKED(UNearestNeighborModelVizSettings, VertVizSectionIndex), UNearestNeighborModelVizSettings::StaticClass());
		TrainingMeshesCategoryBuilder->AddProperty(VertVizHandle).CustomWidget()
		.NameContent()
		[
			VertVizHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			VertVizComboBox
		];
	}

	void FNearestNeighborModelVizSettingsDetails::AddLiveAdditionalSettings()
	{
		IDetailGroup& NNGroup = LiveSettingsCategory->AddGroup("Nearest Neighbor", LOCTEXT("NearestNeighborLabel", "Nearest Neighbor"), false, true);
		NNGroup.AddPropertyRow(DetailLayoutBuilder->GetProperty(GET_MEMBER_NAME_STRING_CHECKED(UNearestNeighborModelVizSettings, NearestNeighborActorSectionIndex), UNearestNeighborModelVizSettings::StaticClass()));
		NNGroup.AddPropertyRow(DetailLayoutBuilder->GetProperty(UNearestNeighborModelVizSettings::GetNearestNeighborIdsPropertyName(), UNearestNeighborModelVizSettings::StaticClass()));	
	}

	FNearestNeighborEditorModel* FNearestNeighborModelVizSettingsDetails::GetCastEditorModel() const
	{
		return static_cast<FNearestNeighborEditorModel*>(EditorModel);
	}
};
#undef LOCTEXT_NAMESPACE
