// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborTrainingModel.h"

#include "Animation/AnimSequence.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "NearestNeighborEditorModel.h"
#include "NearestNeighborModel.h"
#include "NearestNeighborModelInstance.h"
#include "Tools/NearestNeighborKMeansTool.h"
#include "Tools/NearestNeighborStatsTool.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NearestNeighborTrainingModel)

#define LOCTEXT_NAMESPACE "NearestNeighborTrainingModel"

using UE::NearestNeighborModel::FNearestNeighborEditorModel;

UNearestNeighborTrainingModel::~UNearestNeighborTrainingModel() = default;

void UNearestNeighborTrainingModel::Init(UE::MLDeformer::FMLDeformerEditorModel* InEditorModel)
{
	UMLDeformerGeomCacheTrainingModel::Init(InEditorModel);
	check(InEditorModel != nullptr);
	check(InEditorModel->GetModel() != nullptr);
}

UNearestNeighborModel* UNearestNeighborTrainingModel::GetNearestNeighborModel() const
{
	return GetCastModel();
}

int32 UNearestNeighborTrainingModel::GetNumFramesAnimSequence(const UAnimSequence* Anim) const
{
	return UE::NearestNeighborModel::FHelpers::GetNumFrames(Anim);
}

int32 UNearestNeighborTrainingModel::GetNumFramesGeometryCache(const UGeometryCache* GeometryCache) const
{
	return UE::NearestNeighborModel::FHelpers::GetNumFrames(GeometryCache);
}

const USkeleton* UNearestNeighborTrainingModel::GetModelSkeleton(const UMLDeformerModel* Model) const
{
	if (Model)
	{
		if (const USkeletalMesh* const SkeletalMesh = Model->GetSkeletalMesh())
		{
			return SkeletalMesh->GetSkeleton();
		}
	}
	return nullptr;
}


bool UNearestNeighborTrainingModel::SetCustomSamplerData(UAnimSequence* Anim, UGeometryCache* Cache)
{
	if (!CustomSampler.Get())
	{
		SetNewCustomSampler();
		if (!CustomSampler.Get() || !CustomSampler->IsInitialized())
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Failed to initialize CustomSampler. Please provide at least one AnimSequence in TrainingAnimList")
			return false;
		}
	}
	CustomSampler->Customize(Anim, Cache);
	return true;
}

bool UNearestNeighborTrainingModel::CustomSample(int32 Frame)
{
	if (!CustomSampler.Get())
	{
		return false;
	}
	if (!CustomSampler->CustomSample(Frame))
	{
		return false;
	}

	CustomSamplerBoneRotations = CustomSampler->GetBoneRotations();
	CustomSamplerDeltas = CustomSampler->GetVertexDeltas();
	return true;
}

bool UNearestNeighborTrainingModel::SetCustomSamplerDataFromSection(int32 SectionIndex)
{
	if (!CustomSampler.Get())
	{
		SetNewCustomSampler();
		if (!CustomSampler.Get() || !CustomSampler->IsInitialized())
		{
			UE_LOGF(LogNearestNeighborModel, Error, "Failed to initialize CustomSampler. Please provide at least one AnimSequence in TrainingAnimList")
			return false;
		}
	}

	const UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
	if (!NearestNeighborModel)
	{
		return false;
	}

	if (SectionIndex < 0 || SectionIndex >= NearestNeighborModel->GetNumSections())
	{
		return false;
	}

	const UNearestNeighborModel::FSection& Section = NearestNeighborModel->GetSection(SectionIndex);
	UAnimSequence* AnimSequence = Section.GetMutableNeighborPoses();
	UGeometryCache* GeometryCache = Section.GetMutableNeighborMeshes();
	return SetCustomSamplerData(AnimSequence, GeometryCache);
}

namespace UE::NearestNeighborModel::Private
{
	TArray<float> VectorToFloat(TConstArrayView<FVector3f> VectorArr)
	{
		TArray<float> FloatArr;
		FloatArr.SetNumUninitialized(VectorArr.Num() * 3);
		for (int32 i = 0; i < VectorArr.Num(); i++)
		{
			FloatArr[i * 3] = VectorArr[i].X;
			FloatArr[i * 3 + 1] = VectorArr[i].Y;
			FloatArr[i * 3 + 2] = VectorArr[i].Z;
		}
		return FloatArr;
	}
};

TArray<float> UNearestNeighborTrainingModel::GetUnskinnedVertexPositions()
{
	TArray<float> Empty;
	if (!CustomSampler.Get())
	{
		SetNewCustomSampler();
	}
	if (!CustomSampler.Get() || !CustomSampler->IsInitialized())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "Failed to initialize customSampler. Please provide at least one AnimSequence in TrainingAnimList")
		return TArray<float>();
	}
	const TArray<FVector3f>& PositionsVec = CustomSampler->GetUnskinnedVertexPositions();
	return UE::NearestNeighborModel::Private::VectorToFloat(PositionsVec);
}

TArray<int32> UNearestNeighborTrainingModel::GetMeshIndexBuffer()
{
	if (!CustomSampler.Get())
	{
		SetNewCustomSampler();
	}
	if (!CustomSampler.Get() || !CustomSampler->IsInitialized())
	{
		UE_LOGF(LogNearestNeighborModel, Error, "Failed to initialize customSampler. Please provide at least one AnimSequence in TrainingAnimList")
		return TArray<int32>();
	}
	TArray<uint32> UIndexBuffer = CustomSampler->GetMeshIndexBuffer();
	return TArray<int32>((int32*)UIndexBuffer.GetData(), UIndexBuffer.Num());
}

UNearestNeighborModelInstance* UNearestNeighborTrainingModel::CreateModelInstance()
{
	UNearestNeighborModelInstance* const ModelInstance = NewObject<UNearestNeighborModelInstance>(this);
	UNearestNeighborModel* const NearestNeighborModel = GetCastModel();
	if (!NearestNeighborModel)
	{
		return nullptr;
	}
	ModelInstance->SetModel(NearestNeighborModel);
	USkeletalMeshComponent* const DummySKC = NewObject<USkeletalMeshComponent>(ModelInstance);
	DummySKC->SetSkeletalMesh(NearestNeighborModel->GetSkeletalMesh());

	ModelInstance->Init(DummySKC);
	ModelInstance->PostMLDeformerComponentInit();
	return ModelInstance;
}

void UNearestNeighborTrainingModel::DestroyModelInstance(UNearestNeighborModelInstance* ModelInstance)
{
	ModelInstance->ConditionalBeginDestroy();
}

void UNearestNeighborTrainingModel::SetNewCustomSampler()
{
	CustomSampler = MakeUnique<UE::NearestNeighborModel::FNearestNeighborGeomCacheSampler>();
	CustomSampler->Init(EditorModel, 0);
}

UNearestNeighborModel* UNearestNeighborTrainingModel::GetCastModel() const
{
	return Cast<UNearestNeighborModel>(GetModel());
}

UE::NearestNeighborModel::FNearestNeighborEditorModel* UNearestNeighborTrainingModel::GetCastEditorModel() const
{
	return static_cast<UE::NearestNeighborModel::FNearestNeighborEditorModel*>(GetEditorModel());
}


#undef LOCTEXT_NAMESPACE
