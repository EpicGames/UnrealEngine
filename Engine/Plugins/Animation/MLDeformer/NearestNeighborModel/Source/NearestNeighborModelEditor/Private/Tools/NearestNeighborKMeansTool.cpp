// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborKMeansTool.h"

#include "Animation/AnimSequence.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailsViewArgs.h"
#include "GeometryCache.h"
#include "IContentBrowserSingleton.h"
#include "IDetailsView.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "MLDeformerAsset.h"
#include "MLDeformerEditorToolkit.h"
#include "NearestNeighborModel.h"
#include "NearestNeighborTrainingModel.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SButton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NearestNeighborKMeansTool)

#define LOCTEXT_NAMESPACE "NearestNeighborKMeansData"

namespace UE::NearestNeighborModel
{
	namespace Private
	{
		template<class T>
		T* CreateOrLoad(const FString& PackageName)
		{
			const FName AssetName(FPackageName::GetLongPackageAssetName(PackageName));		
			if (UPackage* const Package = CreatePackage(*PackageName))
			{
				LoadPackage(nullptr, *PackageName, LOAD_Quiet | LOAD_EditorOnly);
				T* Asset = FindObject<T>(Package, *AssetName.ToString());
				if (!Asset)
				{
					Asset = NewObject<T>(Package, *AssetName.ToString(), RF_Public | RF_Standalone | RF_Transactional);
					Asset->MarkPackageDirty();
					FAssetRegistryModule::AssetCreated(Asset);
				}
				return Asset;
			}
			return nullptr;
		}

		template<class AssetType>
		TObjectPtr<AssetType> NewAssetDialog(const FString& DefaultPath, const FString& DefaultName)
		{
			FSaveAssetDialogConfig Config;
			{
				Config.DefaultPath = DefaultPath;
				Config.DefaultAssetName = DefaultName;
				Config.AssetClassNames.Add(AssetType::StaticClass()->GetClassPathName());
				Config.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
				Config.DialogTitleOverride = LOCTEXT("ExportAssetTitle", "Export Asset As");
			}

			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				
			FString NewPackageName;
			FText OutError;
			for (bool bFilenameValid = false; !bFilenameValid; bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError))
			{
				const FString AssetPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(Config);
				if (AssetPath.IsEmpty())
				{
					return nullptr;
				}
				NewPackageName = FPackageName::ObjectPathToPackageName(AssetPath);
			}
			return CreateOrLoad<AssetType>(NewPackageName);
		}
	};

	FName FNearestNeighborKMeansTool::GetToolName()
	{
		return FName("Key Pose Extraction Tool");
	}

	FText FNearestNeighborKMeansTool::GetToolTip()
	{
		return LOCTEXT("NearestNeighborKMeansToolTip", "Extract key poses using K-Means clustering.");
	}

	UObject* FNearestNeighborKMeansTool::CreateData()
	{
		return NewObject<UNearestNeighborKMeansData>();
	}

	void FNearestNeighborKMeansTool::InitData(UObject& Data, UE::MLDeformer::FMLDeformerEditorToolkit& Toolkit)
	{
		if (const UMLDeformerAsset* Asset = Toolkit.GetDeformerAsset())
		{
			if (UNearestNeighborKMeansData* KMeansData = Cast<UNearestNeighborKMeansData>(&Data))
			{
				KMeansData->NearestNeighborModelAsset = Asset;
			}
		}
	}

	TSharedRef<SWidget> FNearestNeighborKMeansTool::CreateAdditionalWidgets(UObject& Data, TWeakPtr<UE::MLDeformer::FMLDeformerEditorModel> InEditorModel)
	{
		return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(FText::FromString("New Poses"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([&Data]() -> FReply
				{
					if (UNearestNeighborKMeansData* KMeansData = Cast<UNearestNeighborKMeansData>(&Data))
					{
						if (KMeansData->NearestNeighborModelAsset)
						{
							const FString DefaultPath = FPackageName::GetLongPackagePath(KMeansData->NearestNeighborModelAsset->GetOutermost()->GetName());
							const FString DefaultName = FString::Printf(TEXT("AS_Clusters_%s"), *KMeansData->NearestNeighborModelAsset->GetName());
							UAnimSequence* Asset = Private::NewAssetDialog<UAnimSequence>(DefaultPath, DefaultName);
							KMeansData->ExtractedPoses = Asset;
						}
					}
					return FReply::Handled();
				})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(FText::FromString("New Geometry Cache"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.OnClicked_Lambda([&Data]() -> FReply
				{
					if (UNearestNeighborKMeansData* KMeansData = Cast<UNearestNeighborKMeansData>(&Data))
					{
						if (KMeansData->NearestNeighborModelAsset)
						{
							const FString DefaultPath = FPackageName::GetLongPackagePath(KMeansData->NearestNeighborModelAsset->GetOutermost()->GetName());
							const FString DefaultName = FString::Printf(TEXT("AS_Clusters_%s"), *KMeansData->NearestNeighborModelAsset->GetName());
							UGeometryCache* Asset = Private::NewAssetDialog<UGeometryCache>(DefaultPath, DefaultName);
							KMeansData->ExtractedCache = Asset;
						}
					}
					return FReply::Handled();
				})
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(300)
			[
				SNew(SButton)
				.Text(FText::FromString("Extract"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([&Data, InEditorModel]() -> FReply
				{
					if (TSharedPtr<UE::MLDeformer::FMLDeformerEditorModel> EditorModel = InEditorModel.Pin())
					{
						if (UNearestNeighborKMeansData* KMeansData = Cast<UNearestNeighborKMeansData>(&Data))
						{
							EOpFlag Result = EOpFlag::Success;
							if (!EditorModel->IsReadyForTraining())
							{
								Result = EOpFlag::Error;
								UE_LOGF(LogNearestNeighborModel, Error, "Model is not ready for training. Please check if training data is not empty or reload MLDeformer editor.");
							}
							else
							{
								UNearestNeighborTrainingModel* TrainingModel = FHelpers::NewDerivedObject<UNearestNeighborTrainingModel>();
								TrainingModel->Init(EditorModel.Get());
								const int32 ResultInt = TrainingModel->KmeansClusterPoses(KMeansData);
								Result = ToOpFlag(ResultInt);
							}

							const FText WindowTitle = LOCTEXT("KmeansWindowTitle", "Extraction Results");
							if (OpFlag::HasError(Result))
							{
								FMessageDialog::Open(EAppMsgType::Ok, 
								LOCTEXT("KmeansError", "Failed to extract poses. Please check the Output Log for details."), 
								WindowTitle);
							}
							else if (OpFlag::HasWarning(Result))
							{
								FMessageDialog::Open(EAppMsgType::Ok, 
								LOCTEXT("KmeansWarning", "Extraction finished with warnings. Please check the Output Log for details."), 
								WindowTitle);
							}
						}
					}
					return FReply::Handled();
				})
			]
		];
	}

};

#undef LOCTEXT_NAMESPACE
