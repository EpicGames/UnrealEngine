// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NearestNeighborMenuTool.h"
#include "NearestNeighborModelHelpers.h"
#include "Widgets/SCompoundWidget.h"

#include "NearestNeighborKMeansTool.generated.h"

class UAnimSequence;
class UGeometryCache;
class UMLDeformerAsset;
class UNearestNeighborModel;
namespace UE::MLDeformer
{
	class FMLDeformerEditorModel;
}

class UNearestNeighborKMeansData;

USTRUCT(BlueprintType)
struct FNearestNeighborKMeansInputData
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input")
	TObjectPtr<UAnimSequence> Poses;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input", DisplayName = "(Optional) Cache" )
	TObjectPtr<UGeometryCache> Cache;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input")
	TArray<int> MustIncludeFrames;
};


UCLASS(MinimalAPI, Blueprintable)
class UNearestNeighborKMeansData : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(VisibleAnywhere, Category = "Input")
	TObjectPtr<const UMLDeformerAsset> NearestNeighborModelAsset;

	/** Section used to generate clustered poses. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input")
	int32 SectionIndex = 0;
	
	/** Number of clusters to be generated. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input", meta = (ClampMin = 1))
	int32 NumClusters = 10;

	/** Whether to extract geometry cache at the same time. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input")
	bool bExtractGeometryCache = false;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input")
	TArray<FNearestNeighborKMeansInputData> Inputs;

	/** Extracted poses. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Output")
	TObjectPtr<UAnimSequence> ExtractedPoses;

	/** Extracted geometry cache. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Output", meta = (EditCondition = "bExtractGeometryCache"))
	TObjectPtr<UGeometryCache> ExtractedCache;
};

namespace UE::NearestNeighborModel
{
	class FNearestNeighborKMeansTool : public FNearestNeighborMenuTool
	{
	public:
		virtual ~FNearestNeighborKMeansTool() = default;
		virtual FName GetToolName() override;
		virtual FText GetToolTip() override;
		virtual UObject* CreateData() override;
		virtual void InitData(UObject& Data, UE::MLDeformer::FMLDeformerEditorToolkit& Toolkit) override;
		virtual TSharedRef<SWidget> CreateAdditionalWidgets(UObject& Data, TWeakPtr<UE::MLDeformer::FMLDeformerEditorModel> InEditorModel) override;
	};
};
