// Copyright Epic Games, Inc. All Rights Reserved.

#include "NearestNeighborStatsTool.h"

#include "Animation/AnimSequence.h"
#include "Misc/MessageDialog.h"
#include "MLDeformerAsset.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerEditorToolkit.h"
#include "NearestNeighborTrainingModel.h"
#include "Widgets/Input/SButton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NearestNeighborStatsTool)

#define LOCTEXT_NAMESPACE "NearestNeighborStatsData"

namespace UE::NearestNeighborModel
{
	FName FNearestNeighborStatsTool::GetToolName()
	{
		return FName("Get Neighbor Stats");
	}

	FText FNearestNeighborStatsTool::GetToolTip()
	{
		return LOCTEXT("NearestNeighborStatsToolTip", "Compute nearest neighbor statistics.");
	}

	UObject* FNearestNeighborStatsTool::CreateData()
	{
		return NewObject<UNearestNeighborStatsData>();
	}

	void FNearestNeighborStatsTool::InitData(UObject& Data, UE::MLDeformer::FMLDeformerEditorToolkit& Toolkit)
	{
		if (const UMLDeformerAsset* Asset = Toolkit.GetDeformerAsset())
		{
			if (UNearestNeighborStatsData* StatsData = Cast<UNearestNeighborStatsData>(&Data))
			{
				StatsData->NearestNeighborModelAsset = Asset;
			}
		}
	}

	TSharedRef<SWidget> FNearestNeighborStatsTool::CreateAdditionalWidgets(UObject& Data, TWeakPtr<UE::MLDeformer::FMLDeformerEditorModel> InEditorModel)
	{
		return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(300)
			[
				SNew(SButton)
				.Text(FText::FromString("Get Stats"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([&Data, InEditorModel]() -> FReply
				{
					if (TSharedPtr<UE::MLDeformer::FMLDeformerEditorModel> EditorModel = InEditorModel.Pin())
					{
						if (UNearestNeighborStatsData* StatsData = Cast<UNearestNeighborStatsData>(&Data))
						{
							UNearestNeighborTrainingModel* TrainingModel = FHelpers::NewDerivedObject<UNearestNeighborTrainingModel>();
							TrainingModel->Init(EditorModel.Get());
							const int32 ResultInt = TrainingModel->GetNeighborStats(StatsData);
							const EOpFlag Result = ToOpFlag(ResultInt);
							const FText WindowTitle = LOCTEXT("NeighborStatsWindowTitle", "Stats Results");
							if (OpFlag::HasError(Result))
							{
								FMessageDialog::Open(EAppMsgType::Ok, 
								LOCTEXT("NeighborStatsError", "Failed to compute stats. Please check the Output Log for details."), 
								WindowTitle);
							}
							else if (OpFlag::HasWarning(Result))
							{
								FMessageDialog::Open(EAppMsgType::Ok, 
								LOCTEXT("NeighborStatsWarning", "Finished with warnings. Please check the Output Log for details."), 
								WindowTitle);
							}
						}
					}
					return FReply::Handled();
				})
			]
		];
	}	
};

#undef LOCTEXT_NAMESPACE
