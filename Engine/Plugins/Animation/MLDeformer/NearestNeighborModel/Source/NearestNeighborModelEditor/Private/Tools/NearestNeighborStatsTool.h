// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NearestNeighborMenuTool.h"

#include "NearestNeighborStatsTool.generated.h"

class UAnimSequence;
class UMLDeformerAsset;
namespace UE::MLDeformer
{
	class FMLDeformerEditorModel;
	class FMLDeformerEditorToolkit;
};

UCLASS(MinimalAPI, Blueprintable)
class UNearestNeighborStatsData : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(VisibleAnywhere, Category = "Input")
	TObjectPtr<const UMLDeformerAsset> NearestNeighborModelAsset;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input")
	TObjectPtr<UAnimSequence> TestAnim; 

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Input")
	int32 SectionIndex = 0;
};

namespace UE::NearestNeighborModel
{
	class FNearestNeighborStatsTool : public FNearestNeighborMenuTool
	{
	public:
		virtual ~FNearestNeighborStatsTool() = default;
		virtual FName GetToolName() override;
		virtual FText GetToolTip() override;
		virtual UObject* CreateData() override;
		virtual void InitData(UObject& Data, UE::MLDeformer::FMLDeformerEditorToolkit& Toolkit) override;
		virtual TSharedRef<SWidget> CreateAdditionalWidgets(UObject& Data, TWeakPtr<UE::MLDeformer::FMLDeformerEditorModel> InEditorModel) override;
	};
};
