// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerMorphModelEditorModel.h"
#include "NearestNeighborEditorModelActor.h"
#include "NearestNeighborModelHelpers.h"
#include "Types/SlateEnums.h"

#define UE_API NEARESTNEIGHBORMODELEDITOR_API

class UMLDeformerComponent;
class UNearestNeighborModelSection;
class USkeletalMesh;
class UNearestNeighborModel;
class UNearestNeighborModelInstance;
class UNearestNeighborModelVizSettings;

namespace UE::NearestNeighborModel
{
	class FVertVizSelector;
	class FNearestNeighborEditorModelActor;
	class FVertexMapSelector;

	class FNearestNeighborEditorModel
		: public UE::MLDeformer::FMLDeformerMorphModelEditorModel
	{
	public:
		using FSection = UNearestNeighborModelSection;
		using FMLDeformerSampler = UE::MLDeformer::FMLDeformerSampler;

		// We need to implement this static MakeInstance method.
		static UE_API FMLDeformerEditorModel* MakeInstance();

		// FGCObject overrides.
		virtual FString GetReferencerName() const override { return TEXT("FNearestNeighborEditorModel"); }
		// ~END FGCObject overrides.

		// FMLDeformerEditorModel overrides.
		UE_API virtual void Init(const InitSettings& Settings) override;
		UE_API virtual TSharedPtr<FMLDeformerSampler> CreateSamplerObject() const override;
		UE_API virtual void CreateActors(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene) override;
		UE_API virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
		UE_API virtual void InitInputInfo(UMLDeformerInputInfo* InputInfo) override;
		UE_API virtual ETrainingResult Train() override;
		UE_API virtual bool LoadTrainedNetwork() const override;
		UE_API virtual FMLDeformerTrainingInputAnim* GetTrainingInputAnim(int32 Index) const override;
		UE_API virtual void UpdateTimelineTrainingAnimList() override;
		UE_API virtual void OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent) override;
		UE_API virtual void OnPostTraining(ETrainingResult TrainingResult, bool bUsePartiallyTrainedWhenAborted) override;
		UE_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
		UE_API virtual void UpdateTrainingDeviceList() override;
		UE_API virtual void CopyBaseSettingsFromModel(const FMLDeformerEditorModel* SourceEditorModel) override;
		// ~END FMLDeformerEditorModel overrides.
		
		// UMLDeformerMorphModelEditorModel overrides.
		virtual bool IsMorphWeightClampingSupported() const override	{ return false; }	// We already do input clamping, so output clamping really isn't needed.
		// ~END UMLDeformerMorphModelEditorModel overrides.

		UE_API void OnUpdateClicked();
		UE_API void ClearReferences();

		UE_API FVertexMapSelector* GetVertexMapSelector() const;
		UE_API FVertVizSelector* GetVertVizSelector() const;

	protected:
		UE_API virtual void CreateSamplers() override;
		UE_API virtual bool IsAnimIndexValid(int32 Index) const override; 

	private:
		static constexpr int32 DefaultState = INDEX_NONE; 

		// Some helpers that cast to this model's variants of some classes.
		UE_API UNearestNeighborModel* GetCastModel() const;
		UE_API UNearestNeighborModelVizSettings* GetCastVizSettings() const;
		UE_API UMLDeformerComponent* GetTestMLDeformerComponent() const;
		UE_API USkeletalMeshComponent* GetTestSkeletalMeshComponent() const;
		UE_API UNearestNeighborModelInstance* GetTestNearestNeighborModelInstance() const;
		UE_API void CreateDefaultSection();

		UE_API EOpFlag Update();
		UE_API EOpFlag CheckNetwork();
		UE_API EOpFlag UpdateNearestNeighborData();
		UE_API EOpFlag UpdateMorphDeltas();
		UE_API void UpdateNearestNeighborIds();

		UE_API FNearestNeighborEditorModelActor* CreateNearestNeighborActor(UWorld* World) const;
		UE_API void UpdateNearestNeighborActor(FNearestNeighborEditorModelActor& Actor) const;
	
		FNearestNeighborEditorModelActor* NearestNeighborActor = nullptr;	// This should be only set in CreateActors() and is automatically deleted by the base class.
		TUniquePtr<FVertexMapSelector> VertexMapSelector;
		TUniquePtr<FVertVizSelector> VertVizSelector;
	};

	class FVertexMapSelector
	{
	public:
		void Update(const USkeletalMesh* SkelMesh);
		TArray<TSharedPtr<FString>>* GetOptions();
		void OnSelectionChanged(UNearestNeighborModelSection& Section, TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo) const;
		TSharedPtr<FString> GetSelectedItem(const UNearestNeighborModelSection& Section) const;
		FString GetVertexMapString(const UNearestNeighborModelSection& Section) const;
		bool IsValid() const;

	private:
		void Reset();
		TArray<TSharedPtr<FString>> Options;
		TMap<TSharedPtr<FString>, FString> VertexMapStrings;
		static TSharedPtr<FString> CustomString;
	};

	class FVertVizSelector
	{
	public:
		FVertVizSelector(UNearestNeighborModelVizSettings* InSettings);
		void Update(int32 NumSections);
		TArray<TSharedPtr<FString>>* GetOptions();
		void OnSelectionChanged(TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo) const;
		TSharedPtr<FString> GetSelectedItem() const;
		int32 GetSectionIndex(TSharedPtr<FString> Item) const;
		void SelectSection(int32 SectionIndex);

	private:
		TObjectPtr<UNearestNeighborModelVizSettings> Settings;
		TArray<TSharedPtr<FString>> Options;
	};
}	// namespace UE::NearestNeighborModel

#undef UE_API
