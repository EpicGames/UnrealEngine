// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MLDeformerGeomCacheActor.h"

class UGeometryCache;
class UMLDeformerComponent;

namespace UE::NearestNeighborModel
{
	enum : int32
	{
		ActorID_NearestNeighborActors = 6
	};

	class FNearestNeighborEditorModelActor
		: public UE::MLDeformer::FMLDeformerGeomCacheActor
	{
	public:
		FNearestNeighborEditorModelActor(const FConstructSettings& Settings);
		virtual ~FNearestNeighborEditorModelActor();

		void SetGeometryCache(UGeometryCache* InGeometryCache) const;
		void SetTrackedComponent(const UMLDeformerComponent* InComponent, int32 InSectionIndex);
		void Tick() const;
	
	private:
		int32 SectionIndex = INDEX_NONE;
		TWeakObjectPtr<const UMLDeformerComponent> TrackedComponent;
	};
}	// namespace UE::NearestNeighborModel
