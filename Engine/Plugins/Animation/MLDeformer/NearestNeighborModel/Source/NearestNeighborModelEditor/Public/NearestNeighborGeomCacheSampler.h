// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MLDeformerGeomCacheSampler.h"
#include "NearestNeighborModelHelpers.h"

#define UE_API NEARESTNEIGHBORMODELEDITOR_API

class UAnimSequence;
class UGeometryCache;
class UNearestNeighborTrainingModel;

namespace UE::NearestNeighborModel
{
	class FNearestNeighborEditorModel;

	class FNearestNeighborGeomCacheSampler
		: public UE::MLDeformer::FMLDeformerGeomCacheSampler
	{
	public:
		// FMLDeformerGeomCacheSampler overrides
		UE_API virtual void Sample(int32 InAnimFrameIndex) override;
		// ~END FMLDeformerGeomCacheSampler overrides

		/** This will set the SkeletalMeshComponent with custom Anim and GeometryCacheComponent with custom Cache. 
		 * Using this fuction will stop using Anim and Cache from the EditorModel.
		 * Using this function will break Sample(). Call CustomSample() instead.
		 * Do NOT use this function on EditorModel Samplers. 
		 */ 
		UE_API void Customize(UAnimSequence* Anim, UGeometryCache* Cache = nullptr);

		/**
		 * Sample function used with Customize
		 * @return true if sampling is successful
		 */
		UE_API bool CustomSample(int32 Frame);

		/**
		 * Get the Mesh Index Buffer of the skeletal mesh object used by the model.
		 * 
		 * @return A flattened array of the index buffer.
		 */
		UE_API TArray<uint32> GetMeshIndexBuffer() const;

	private:
		UE_API void SampleDualQuaternionDeltas(int32 InAnimFrameIndex);
		UE_API EOpFlag GenerateMeshMappings();
		UE_API EOpFlag CheckMeshMappingsEmpty() const;
		UE_API FVector3f CalcDualQuaternionDelta(int32 VertexIndex, const FVector3f& WorldDelta, const FSkeletalMeshLODRenderData& SkelMeshLODData, const FSkinWeightVertexBuffer& SkinWeightBuffer) const;
	};
}

#undef UE_API
