// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerMorphModelDetails.h"

#define UE_API NEARESTNEIGHBORMODELEDITOR_API

class IDetailChildrenBuilder;
class IPropertyHandle;
class UNearestNeighborModel;

namespace UE::NearestNeighborModel
{
	class FNearestNeighborEditorModel;

	class FNearestNeighborModelDetails
		: public ::UE::MLDeformer::FMLDeformerMorphModelDetails
	{
	public:
		/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
		static UE_API TSharedRef<IDetailCustomization> MakeInstance();

		// ILayoutDetails overrides.
		UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
		UE_API virtual void CreateCategories() override;
		// ~END ILayoutDetails overrides.

		// FMLDeformerModelDetails overrides.
		UE_API virtual bool UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects) override;
		// ~END FMLDeformerModelDetails overrides.

	private:
		UE_API UNearestNeighborModel* GetCastModel() const;
		UE_API FNearestNeighborEditorModel* GetCastEditorModel() const;

		UE_API void CustomizeTrainingSettingsCategory() const;
		UE_API void CustomizeNearestNeighborSettingsCategory() const;
		UE_API void CustomizeSectionsCategory(IDetailLayoutBuilder& DetailBuilder);
		UE_API void CustomizeMorphTargetCategory(IDetailLayoutBuilder& DetailBuilder) const;
		UE_API void CustomizeStatusCategory() const;
		UE_API void CustomizeFileCacheCategory(IDetailLayoutBuilder& DetailBuilder) const;

		UE_API void GenerateSectionElementWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder);

		IDetailCategoryBuilder* NearestNeighborCategoryBuilder = nullptr;
		IDetailCategoryBuilder* StatusCategoryBuilder = nullptr;
		IDetailCategoryBuilder* SectionsCategoryBuilder = nullptr;
	};
}	// namespace UE::NearestNeighborModel

#undef UE_API
