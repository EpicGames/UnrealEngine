// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

#define UE_API NEARESTNEIGHBORMODELEDITOR_API

namespace UE::NearestNeighborModel
{
	class FNearestNeighborModelEditorStyle
		: public FSlateStyleSet
	{
	public:
		UE_API FNearestNeighborModelEditorStyle();
		UE_API virtual ~FNearestNeighborModelEditorStyle();

		static UE_API FNearestNeighborModelEditorStyle& Get();
	};

}	// namespace UE::NearestNeighborModel

#undef UE_API
