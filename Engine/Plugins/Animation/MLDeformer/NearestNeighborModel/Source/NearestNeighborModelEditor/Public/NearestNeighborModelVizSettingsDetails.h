// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MLDeformerMorphModelVizSettingsDetails.h"

#define UE_API NEARESTNEIGHBORMODELEDITOR_API

namespace UE::NearestNeighborModel
{
	class FNearestNeighborEditorModel;
	class FNearestNeighborModelVizSettingsDetails
		: public UE::MLDeformer::FMLDeformerMorphModelVizSettingsDetails
	{
	public:
		/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
		static UE_API TSharedRef<IDetailCustomization> MakeInstance();
		UE_API virtual void AddAdditionalSettings() override;

	private:
		UE_API void AddTrainingMeshAdditionalSettings();
		UE_API void AddLiveAdditionalSettings();
		UE_API FNearestNeighborEditorModel* GetCastEditorModel() const;
	};
}	// namespace UE::NearestNeighborModel

#undef UE_API
