// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MLDeformerGeomCacheTrainingModel.h"
#include "NearestNeighborGeomCacheSampler.h"

#include "NearestNeighborTrainingModel.generated.h"

#define UE_API NEARESTNEIGHBORMODELEDITOR_API

namespace UE::NearestNeighborModel
{
	class FNearestNeighborEditorModel;
}

class UAnimSequence;
class UMLDeformerModel;
class UNearestNeighborKMeansData;
class UNearestNeighborModel;
class UNearestNeighborStatsData;
class UNearestNeighborModelInstance;

UCLASS(MinimalAPI, Blueprintable)
class UNearestNeighborTrainingModel
	: public UMLDeformerGeomCacheTrainingModel
{
	GENERATED_BODY()

public:
	UE_API virtual ~UNearestNeighborTrainingModel();
	UE_API virtual void Init(UE::MLDeformer::FMLDeformerEditorModel* InEditorModel) override;

	UFUNCTION(BlueprintImplementableEvent, Category = "Python")
	UE_API int32 Train() const;

	UFUNCTION(BlueprintImplementableEvent, Category = "Python")
	UE_API int32 UpdateNearestNeighborData();

	UFUNCTION(BlueprintImplementableEvent, Category = "Python")
	UE_API int32 KmeansClusterPoses(const UNearestNeighborKMeansData* KMeansData);

	UFUNCTION(BlueprintImplementableEvent, Category = "Python")
	UE_API bool GetNeighborStats(const UNearestNeighborStatsData* StatsData);

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Python")
	TArray<float> CustomSamplerBoneRotations;
	
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Python")
	TArray<float> CustomSamplerDeltas;

private:
	UFUNCTION(BlueprintPure, Category = "Python")
	UE_API int32 GetNumFramesAnimSequence(const UAnimSequence* Anim) const;

	UFUNCTION(BlueprintPure, Category = "Python")
	UE_API int32 GetNumFramesGeometryCache(const UGeometryCache* GeometryCache) const;
	
	UFUNCTION(BlueprintPure, Category = "Python")
	UE_API const USkeleton* GetModelSkeleton(const UMLDeformerModel* Model) const;
	
	UFUNCTION(BlueprintPure, Category = "Python")
	UE_API UNearestNeighborModel* GetNearestNeighborModel() const;

	UFUNCTION(BlueprintCallable, Category = "Python")
	UE_API bool SetCustomSamplerData(UAnimSequence* Anim, UGeometryCache* Cache = nullptr);

	UFUNCTION(BlueprintCallable, Category = "Python")
	UE_API bool CustomSample(int32 Frame);

	UFUNCTION(BlueprintCallable, Category = "Python")
	UE_API bool SetCustomSamplerDataFromSection(int32 SectionIndex);

	UFUNCTION(BlueprintPure, Category = "Python")
	UE_API TArray<float> GetUnskinnedVertexPositions();

	UFUNCTION(BlueprintPure, Category = "Python")
	UE_API TArray<int32> GetMeshIndexBuffer();

	UFUNCTION(BlueprintCallable, Category = "Python")
	UE_API UNearestNeighborModelInstance* CreateModelInstance();

	UFUNCTION(BlueprintCallable, Category = "Python")
	UE_API void DestroyModelInstance(UNearestNeighborModelInstance* ModelInstance);

	UE_API void SetNewCustomSampler();
	UE_API UNearestNeighborModel* GetCastModel() const;
	UE_API UE::NearestNeighborModel::FNearestNeighborEditorModel* GetCastEditorModel() const;

	TUniquePtr<UE::NearestNeighborModel::FNearestNeighborGeomCacheSampler> CustomSampler;
};

#undef UE_API
